// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.tests;

import android.app.Activity;
import android.os.Bundle;
import android.util.Log;

import java.io.File;
import java.io.FileInputStream;
import java.io.IOException;
import java.util.Arrays;
import java.util.ArrayList;

import androidx.core.app.ActivityCompat;

import android.content.pm.PackageManager;
import android.Manifest.permission;

public class TestActivity extends Activity {
	private int testsReturnCode = -1;

	public TestActivity() {
	}

	private String readCmdLineFrom(String path) {
		try {
			File cmdLineFile = new File(path);
			FileInputStream cmdLineInputStream = new FileInputStream(cmdLineFile);
			byte[] stringBytes = new byte[(int) cmdLineFile.length()];
			if (cmdLineInputStream.read(stringBytes) != stringBytes.length) {
				return "";
			}
			cmdLineInputStream.close();
			return new String(stringBytes);
		} catch (IOException e) {
			Log.e("UE", "Could not process command line, exception: " + e.getMessage());
			return "";
		}
	}

	@Override
	public void onCreate(Bundle savedInstanceState) {
		super.onCreate(savedInstanceState);

		System.loadLibrary("Unreal");

		String PackageName = null;
		try
		{
			PackageName = String.valueOf(getPackageManager().getPackageInfo(getPackageName(), 0).packageName);
		}
		catch (PackageManager.NameNotFoundException e)
		{
			Log.d("UE", "Failed to read package name: " + e.getMessage());
		}

		if (PackageName != null)
		{
			String[] PackageNameParts = PackageName.split("\\.");
			String PackageNameLastPart = PackageNameParts[PackageNameParts.length - 1];

			String GamePath = getExternalFilesDir(null).getAbsolutePath() + "/UnrealGame/" + PackageNameLastPart;
			String cmdLinePath = GamePath + "/UECommandLine.txt";

			String TestCommandLine = readCmdLineFrom(cmdLinePath);
			String AbsolutePath = getFilesDir().getAbsolutePath() + "/";
			String[] AllArgs = TestCommandLine.trim().split("\\s+");

			Log.d("UE", "Command line is " + TestCommandLine);

			testsReturnCode = runTests(AbsolutePath, AllArgs);
		}

		// Required for test status until we can generate reports on Android
		Log.d("UE", "Tests finished with exit code " + testsReturnCode);

		try {
			// Waiting here for any test cleanup seems to be required to prevent potential hang after System.exit()...
			Thread.sleep(3000);
		} catch (InterruptedException ie)
		{}

		finish();

		return;
	}

	@Override
	public void onDestroy()
	{
		super.onDestroy();
		System.exit(testsReturnCode);

	}

	public static native int runTests(String path, String[] args);
}
