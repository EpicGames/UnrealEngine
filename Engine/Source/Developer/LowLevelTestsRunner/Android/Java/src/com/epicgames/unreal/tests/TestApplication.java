// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.tests;

import android.app.Application;
import android.content.Context;
import android.content.res.Configuration;
import androidx.lifecycle.Lifecycle;
import androidx.lifecycle.LifecycleObserver;
import androidx.lifecycle.OnLifecycleEvent;
import androidx.lifecycle.ProcessLifecycleOwner;

import com.epicgames.unreal.IAndroidFileServerSetup;

public class TestApplication extends Application implements LifecycleObserver, IAndroidFileServerSetup {
	private static boolean isForeground = false;

	private static Context context;

	@Override
	public void onCreate() {
		super.onCreate();
		TestApplication.context = getApplicationContext();
		ProcessLifecycleOwner.get().getLifecycle().addObserver(this);
	}

	public static Context getAppContext() {
		return TestApplication.context;
	}

	@Override
	public void attachBaseContext(Context base) {
		super.attachBaseContext(base);
	}

	@Override
	public void onLowMemory() {
		super.onLowMemory();
	}

	@Override
	public void onTrimMemory(int level) {
		super.onTrimMemory(level);
	}

	@Override
	public void onConfigurationChanged(Configuration newConfig) {
		super.onConfigurationChanged(newConfig);
	}

	@OnLifecycleEvent(Lifecycle.Event.ON_START)
	void onEnterForeground() {
		isForeground = true;
	}

	@OnLifecycleEvent(Lifecycle.Event.ON_STOP)
	void onEnterBackground() {
		isForeground = false;
	}

	@SuppressWarnings("unused")
	public static boolean isAppInForeground() {
		return isForeground;
	}

	public static boolean isAppInBackground() {
		return !isForeground;
	}
	
	public boolean AndroidFileServer_Verify(String Token)
	{
		return true;
	}
	
	public boolean AndroidFileServer_Init(String filename)
	{
		return true;
	}
}
