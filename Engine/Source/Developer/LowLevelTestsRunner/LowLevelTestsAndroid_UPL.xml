<?xml version="1.0" encoding="utf-8"?>
<!-- Low level tests UPL script -->
<!-- Replaces application and activity elements, removes unnecessary elements, replaces Java /src with low level tests Java source -->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Low level tests UPL script"/>
	</init>
	<androidManifestUpdates>
		<removeElement tag="application"/>
		<setElement result="TestApplication" value="application" />
		<addAttribute tag="$TestApplication" name="android:name" value="com.epicgames.unreal.tests.TestApplication" />
		<addAttribute tag="$TestApplication" name="android:label" value="@string/app_name" />
		<addAttribute tag="$TestApplication" name="android:icon" value="@drawable/icon" />
		<addAttribute tag="$TestApplication" name="android:hardwareAccelerated" value="true" />
		<addAttribute tag="$TestApplication" name="android:extractNativeLibs" value="true" />
		<addAttribute tag="$TestApplication" name="android:requestLegacyExternalStorage" value="true" />
		<addAttribute tag="$TestApplication" name="android:hasCode" value="true" />
		<addElement tag="manifest" name="TestApplication"/>

		<setElement result="TestActivity" value="activity" />
		<addAttribute tag="$TestActivity" name="android:name" value="com.epicgames.unreal.tests.TestActivity" />
		<addAttribute tag="$TestActivity" name="android:exported" value="true" />
		<addAttribute tag="$TestActivity" name="android:label" value="@string/app_name" />
		<addAttribute tag="$TestActivity" name="android:theme" value="@android:style/Theme.Black.NoTitleBar.Fullscreen" />
		<addAttribute tag="$TestActivity" name="android:launchMode" value="singleTask" />
		<addAttribute tag="$TestActivity" name="android:debuggable" value="true" />
		<addElement tag="application" name="TestActivity"/>

		<setElement result="TestActivityMetaData" value="meta-data" />
		<addAttribute tag="$TestActivityMetaData" name="android:name" value="android.app.lib_name" />
		<addAttribute tag="$TestActivityMetaData" name="android:value" value="Unreal" />
		<addElement tag="activity" name="TestActivityMetaData" />

		<setElement result="TestActivityAction" value="action" />
		<addAttribute tag="$TestActivityAction" name="android:name" value="android.intent.action.MAIN" />

		<setElement result="TestActivityCategory" value="category" />
		<addAttribute tag="$TestActivityCategory" name="android:name" value="android.intent.category.LAUNCHER" />

		<setElement result="TestActivityIntentFilter" value="intent-filter" />
		<addElement tag="activity" name="TestActivityIntentFilter" />

		<addElement tag="intent-filter" name="TestActivityAction" />
		<addElement tag="intent-filter" name="TestActivityCategory" />

		<addPermission android:name="android.permission.INTERNET"/>
		<addPermission android:name="android.permission.READ_EXTERNAL_STORAGE"/>
		<addPermission android:name="android.permission.WRITE_EXTERNAL_STORAGE"/>

		<addPermission android:name="android.permission.FOREGROUND_SERVICE"/>
		<addPermission android:name="android.permission.FOREGROUND_SERVICE_DATA_SYNC"/>
		<addPermission android:name="android.permission.POST_NOTIFICATIONS"/>

		<addElements tag="application">
			<service android:name="com.epicgames.unreal.RemoteFileManagerService" android:foregroundServiceType="dataSync" android:exported="true" />
			<activity android:name="com.epicgames.unreal.RemoteFileManagerActivity" android:exported="true" android:screenOrientation="sensor" android:theme="@android:style/Theme.NoDisplay">
				<intent-filter>
					<action android:name="com.epicgames.unreal.RemoteFileManager.intent.COMMAND2" />
				</intent-filter>
			</activity>
		</addElements>
	</androidManifestUpdates>
	<resourceCopies>
		<isArch arch="arm64-v8a">
			<deleteFiles filespec="arm64/src/*.*" recursive="true" />
		</isArch>
		<isArch arch="x86_64">
			<deleteFiles filespec="x64/src/*.*" recursive="true" />
		</isArch>
		<copyDir src="$S(PluginDir)/Android/Java/src" dst="$S(BuildDir)/src"/>
		<copyDir src="$S(EngineDir)/Plugins/Runtime/AndroidFileServer/Source/AndroidFileServer/Android/java" dst="$S(BuildDir)/src" />
	</resourceCopies>
</root>
