// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class LowLevelTestsRunner : ModuleRules
	{
		public LowLevelTestsRunner(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PCHUsage = PCHUsageMode.NoPCHs;
			PrecompileForTargets = PrecompileTargetsType.None;
			bUseUnity = false;
			bRequiresImplementModule = false;

#if WITH_CATCH2
			string Catch2Version = Catch2.Catch2Version;
#else
			string Catch2Version = "v3.11.0";
#endif

			PublicIncludePaths.AddRange(
				new string[]
				{
					Path.Combine(Target.UEThirdPartySourceDirectory, "Catch2", Catch2Version, "src")
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core"
				}
			);

			if (TestTargetRules.bTestsRequireEditor)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
			if (TestTargetRules.bTestsRequireEngine == true)
			{
				PrivateDependencyModuleNames.Add("Engine");
			}
			if (TestTargetRules.bTestsRequireApplicationCore)
			{
				PrivateDependencyModuleNames.Add("ApplicationCore");
			}
			if (TestTargetRules.bTestsRequireCoreUObject)
			{
				PrivateDependencyModuleNames.Add("CoreUObject");
			}

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string ModulePath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(ModulePath, "LowLevelTestsAndroid_UPL.xml"));
			}
		}
	}
}
