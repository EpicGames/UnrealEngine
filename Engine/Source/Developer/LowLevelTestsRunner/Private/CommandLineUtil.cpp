// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandLineUtil.h"
#include <stdlib.h>
#include <stdio.h>
#include <string.h>

#define MAX_COMMAND_LINE_ARGUMENTS_NUM 64
#define MAX_ADDITIONAL_ARGUMENT_SIZE 128

#define ADDITIONAL_COMMANDLINE_ARGS_FILE "UE5CommandLine.txt"

#ifdef _WIN32
#define safe_strcpy(Dest, DestSize, Source) strcpy_s(Dest, DestSize, Source)
#else
#define safe_strcpy(Dest, DestSize, Source) strcpy(Dest, Source)
#endif

/** 
 * Read command line arguments from file to add on top of the real ones.
 * Specify valid directory with separator at the end and path real command line arguments. 
 */
const char** ReadAndAppendAdditionalArgs(const char* FileDir, int* OutNumArgs, const char** Argv, int Argc)
{
	if (!OutNumArgs || !Argv || !FileDir)
	{
		return NULL;
	}

	if (Argc >= MAX_COMMAND_LINE_ARGUMENTS_NUM)
	{
		return NULL;
	}

	const size_t DirStrLength = strlen(FileDir);
	const size_t FilePathLength = DirStrLength + strlen(ADDITIONAL_COMMANDLINE_ARGS_FILE) + 1;
	char* FilePath = (char*) malloc(FilePathLength);
	safe_strcpy(FilePath, FilePathLength, FileDir);
	safe_strcpy(FilePath + DirStrLength, FilePathLength - DirStrLength, ADDITIONAL_COMMANDLINE_ARGS_FILE);

#ifdef _WIN32
	FILE* CommandLineFile = NULL;
	fopen_s(&CommandLineFile, FilePath, "r");
#else
	FILE* CommandLineFile = fopen(FilePath, "r");
#endif
	free(FilePath);
	FilePath = NULL;
	if (!CommandLineFile)
	{
		return NULL;
	}

	char Buffer[4096];
	const size_t BufferSize = sizeof(Buffer);

	//text has to be in UTF8 and with no BOM, just read first line
	char* AdditionalConsoleArgsLine = fgets(Buffer, BufferSize, CommandLineFile);
	if (!AdditionalConsoleArgsLine)
	{
		return NULL;
	}

	const char Delimiters[] = { ' ', '\t', '\n', '\r', '\0' };

	static char Arguments[MAX_COMMAND_LINE_ARGUMENTS_NUM][MAX_ADDITIONAL_ARGUMENT_SIZE];
	static char* ArgPtrs[MAX_COMMAND_LINE_ARGUMENTS_NUM];

	for (size_t ArgIndex = 0; ArgIndex < MAX_COMMAND_LINE_ARGUMENTS_NUM; ++ArgIndex)
	{
		Arguments[ArgIndex][0] = '\0';
		ArgPtrs[ArgIndex] = &Arguments[ArgIndex][0];
	}

	int NumArguments = Argc;

	//Copy normal args
	for (int ExistingArgIndex = 0; ExistingArgIndex < NumArguments; ++ExistingArgIndex)
	{
		safe_strcpy(Arguments[ExistingArgIndex], sizeof(char) * MAX_ADDITIONAL_ARGUMENT_SIZE, Argv[ExistingArgIndex]);
	}

#ifdef _WIN32
	char* NextToken = NULL;
	char* Token = strtok_s(AdditionalConsoleArgsLine, Delimiters, &NextToken);
	while (Token != NULL && NumArguments < MAX_COMMAND_LINE_ARGUMENTS_NUM)
	{
		safe_strcpy(Arguments[NumArguments], sizeof(char) * MAX_ADDITIONAL_ARGUMENT_SIZE, Token);
		++NumArguments;
		Token = strtok_s(NULL, Delimiters, &NextToken);
	}
#else
	char* Token = strtok(AdditionalConsoleArgsLine, Delimiters);
	while (Token != NULL && NumArguments < MAX_COMMAND_LINE_ARGUMENTS_NUM)
	{
		safe_strcpy(Arguments[NumArguments], sizeof(char) * MAX_ADDITIONAL_ARGUMENT_SIZE, Token);
		++NumArguments;
		Token = strtok(NULL, Delimiters);
	}
#endif

	*OutNumArgs = NumArguments;

	return (const char**)ArgPtrs;
}

#undef safe_strcpy
#undef MAX_COMMAND_LINE_ARGUMENTS_NUM
#undef MAX_ADDITIONAL_ARGUMENT_SIZE
#undef ADDITIONAL_COMMANDLINE_ARGS_FILE