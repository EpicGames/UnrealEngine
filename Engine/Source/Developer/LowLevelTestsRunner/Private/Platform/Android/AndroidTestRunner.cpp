// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(PLATFORM_ANDROID)
#include "TestRunner.h"

#include <jni.h>
#include <string>
#include <vector>

const char* AndroidPath;

extern "C" int LowLevelTestsMain(int argc, const char* argv[])
{
	return RunTests(argc, argv);
}

const char* GetCacheDirectory()
{
	return AndroidPath;
}

const char* GetProcessExecutablePath()
{
	return AndroidPath;
}

extern "C"
{
	// Maps to runTests in TestActivity.java
	JNIEXPORT jint JNICALL
		Java_com_epicgames_unreal_tests_TestActivity_runTests(
			JNIEnv* env,
			jobject jobj, jstring jpath, jobjectArray jargs)
	{
		jboolean isCopy;
		const char* Path = env->GetStringUTFChars(jpath, &isCopy);

		const jsize jargsCount = env->GetArrayLength(jargs);

		TUniquePtr<const char* []> Args = MakeUnique<const char* []>(jargsCount + 1);
		if (Path != nullptr)
		{
			Args[0] = AndroidPath = Path;
		}

		for (int i = 0; i < jargsCount; i++) {
			jboolean isArgCopy;
			jstring jarg = (jstring)env->GetObjectArrayElement(jargs, i);
			const char* Arg = env->GetStringUTFChars(jarg, &isArgCopy);
			char* ArgCpy = new char[strlen(Arg)];
			if (Arg != nullptr)
			{
				strcpy(ArgCpy, Arg);
				Args[i + 1] = ArgCpy;
			}
			env->ReleaseStringUTFChars(jarg, Arg);
		}

		int MainReturn = LowLevelTestsMain(jargsCount + 1, Args.Get());

		env->ReleaseStringUTFChars(jpath, Path);

		return MainReturn;
	}

	JNIEXPORT jint JNI_OnLoad(JavaVM* InJavaVM, void* InReserved)
	{
		return JNI_VERSION_1_6;
	}
}

#endif