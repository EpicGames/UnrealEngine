// Copyright Epic Games, Inc. All Rights Reserved.
#include "AppleTestRunnerHelper.h"

#if PLATFORM_IOS || PLATFORM_MAC

#pragma push_macro("FVector")
#define FVector FVectorWorkaround

#import <CoreFoundation/CFRunLoop.h>
#import <Foundation/Foundation.h>

#undef FVector

#pragma pop_macro("FVector")

#include "TestRunner.h"

@implementation AppleTestsRunnerHelper

int Argc;
const char** Argv;
NSThread* TestsThread;
int _Result;

@synthesize Result = _Result;

-(id)initWithArgc:(int)argc Argv:(const char*[])argv
{
	[super init];
	
	Argc = argc;
	Argv = argv;
	return self;
}

- (void)startTestsOnThread
{
	TestsThread = [[NSThread alloc] initWithTarget:self selector: @selector(runTestsThread:) object: nil];
	[TestsThread start];
}

-(void)dealloc
{
	[TestsThread release];

	[super dealloc];
}

- (void)runTestsThread:(id)Arg
{
	_Result = RunTests(Argc, Argv);
	
	CFRunLoopStop(CFRunLoopGetMain());
}

@end

#endif
