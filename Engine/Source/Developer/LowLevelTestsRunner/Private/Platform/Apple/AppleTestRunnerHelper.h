// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef PLATFORM_IOS
#define PLATFORM_IOS 0
#endif

#ifndef PLATFORM_MAC
#define PLATFORM_MAC 0
#endif

#if PLATFORM_IOS || PLATFORM_MAC

#import <Foundation/NSObject.h>

@interface AppleTestsRunnerHelper : NSObject

@property(readonly) int Result;

-(id)initWithArgc:(int)argc Argv:(const char*[])argv;
-(void)startTestsOnThread;

@end

#endif
