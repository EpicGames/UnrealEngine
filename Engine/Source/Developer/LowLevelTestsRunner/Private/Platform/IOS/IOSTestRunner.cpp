// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_IOS

#include "TestRunner.h"
#include "Platform/Apple/AppleTestRunnerHelper.h"
#include <mach-o/dyld.h>
#import <CoreFoundation/CFRunLoop.h>

const char* GetCacheDirectory()
{
	return "/private/var/tmp/unit_tests_pds_cache";
}

const char* GetProcessExecutablePath()
{
	static char Path[512] = { 0 };
	uint32_t PathSize = sizeof(Path);
	if (_NSGetExecutablePath(Path, &PathSize) == 0)
	{
		//Add extra slash
		PathSize = strlen(Path);
		if (PathSize < sizeof(Path) - 2)
		{
			Path[PathSize] = '/';
			Path[PathSize + 1] = '\0';
			return Path;
		}
	}

	return NULL;
}

int main(int argc, const char* argv[])
{
	AppleTestsRunnerHelper* Helper = [[AppleTestsRunnerHelper alloc] initWithArgc:argc Argv:argv];
	
	[Helper startTestsOnThread];
	
	CFRunLoopRun();
	
	int Result = Helper.Result;
	[Helper release];
	return Result;
}
#endif
