// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_IOS

#if WITH_APPLICATION_CORE
#include "Containers/UnrealString.h"

// Empty implementation of missing symbols used in ApplicationCore but implemented somewhere else (in Launch for instance)
FString GSavedCommandLine;

@class IOSAppDelegate;
@class UIApplication;

namespace FAppEntry
{
	void PlatformInit() {}
	void PreInit(IOSAppDelegate* AppDelegate, UIApplication* Application) {}
	void Init() {}
	void Tick() {}
    void SuspendTick() {}
	void ResumeAudioContext() {}
	void ResetAudioContextResumeTime() {}
	void Shutdown() {}
    void Suspend(bool bIsInterrupt = false) {}
    void Resume(bool bIsInterrupt = false) {}
	void RestartAudio() {}
    void IncrementAudioSuspendCounters() {}
    void DecrementAudioSuspendCounters() {}

	bool IsStartupMoviePlaying() { return false; }

	bool	gAppLaunchedWithLocalNotification;
	FString	gLaunchLocalNotificationActivationEvent;
	int32	gLaunchLocalNotificationFireDate;
}
#else // !WITH_APPLICATION_CORE

// Empty implementation of missing symbols used in Core but implemented in ApplicationCore
#include "Logging/LogMacros.h"
#include "HAL/PlatformMisc.h"

DEFINE_LOG_CATEGORY_STATIC(LogLowLevelTests, Log, VeryVerbose);

// IOSAppDelegate is mostly used in IOSPlatformMisc implementation. It is always accessed using [IOSAppDelegate GetDelegate]
@interface IOSAppDelegate : NSObject
{
}
@end

@implementation IOSAppDelegate
// Implementation returns nil so any method call on that nil will become a no-op. Not having this method would produce a 
// "Sending unrecognized selector to object" uncaught exception in any method in FIOSPlatformMisc that tries to get the delegate  
+ (IOSAppDelegate*)GetDelegate
{
	UE_LOGF(LogLowLevelTests, Error, "Attempt to access to IOSAppDelegate methods. Those are only available if ApplicationCore is present");
	return nil;
}
@end

EAppReturnType::Type MessageBoxExtImpl( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
	return EAppReturnType::Type::Yes;
}

#endif

#endif
