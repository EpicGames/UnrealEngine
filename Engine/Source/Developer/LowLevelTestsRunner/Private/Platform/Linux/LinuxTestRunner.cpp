// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(PLATFORM_LINUX)
#include "TestRunner.h"

#include "Misc/OutputDeviceDebug.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/PlatformOutputDevices.h"
#include "CoreGlobals.h"

#include <unistd.h>

const char* GetProcessExecutablePath()
{
	static char Buffer[512] = { 0 };
	ssize_t Result = readlink("/proc/self/exe", Buffer, sizeof(Buffer));
	if (Result > 0 && Result < (sizeof(Buffer) - 1))
	{
		Buffer[Result] = '\0';

		//Cut off the module file
		for (int Index = (int)Result; Index >= 0; --Index)
		{
			if (Buffer[Index] == '/')
			{
				Buffer[Index + 1] = '\0';
				break;
			}
		}

		return Buffer;
	}

	return nullptr;
}

const char* GetCacheDirectory()
{
	return "/var/tmp/playground_test_pds_cache";
}

void InitializeLinuxPlatform()
{
	GLog->AddOutputDevice(new FOutputDeviceDebug());
}

int main(int argc, const char* argv[])
{
	FTestDelegates::GetGlobalPlatformSetup().BindStatic(&InitializeLinuxPlatform);
	return RunTests(argc, argv);
}
#endif