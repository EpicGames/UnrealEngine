// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_MAC

#include "TestRunner.h"
#include "Mac/MacSystemIncludes.h"
#include "Platform/Apple/AppleTestRunnerHelper.h"
#include <mach-o/dyld.h>

const char* GetCacheDirectory()
{
	return "/private/var/tmp/basic_test_pds_cache";
}

const char* GetProcessExecutablePath()
{
	static char Path[512] = { 0 };
	uint32_t PathSize = sizeof(Path);
	if (_NSGetExecutablePath(Path, &PathSize) == 0)
	{
		//Add extra slash
		PathSize = strlen(Path);
		if (PathSize < sizeof(Path) - 2)
		{
			Path[PathSize] = '/';
			Path[PathSize + 1] = '\0';
			return Path;
		}
	}

	return NULL;
}

int main(int argc, const char* argv[])
{
	AppleTestsRunnerHelper* Helper = [[AppleTestsRunnerHelper alloc] initWithArgc:argc Argv:argv];
	
	[Helper startTestsOnThread];
	
	CFRunLoopRun();
	
	int Result = Helper.Result;
	[Helper release];

	if (GGameThreadId != FPlatformTLS::GetCurrentThreadId())
	{
		// Force Game thread to be the current one for proper cleanup at exit
		FTaskTagScope Scope(ETaskTag::EGameThread);
		GGameThreadId = FPlatformTLS::GetCurrentThreadId();
	}

	return Result;
}
#endif
