// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(PLATFORM_WINDOWS)
#include "TestRunner.h"

const char* GetProcessExecutablePath()
{
	static char Buffer[512] = { 0 };
	const int Result = GetModuleFileNameA(NULL, Buffer, 512);
	if (Result != 0)
	{
		//Cut off the module file
		for (int Index = Result; Index >= 0; --Index)
		{
			if (Buffer[Index] == '/' || Buffer[Index] == '\\')
			{
				Buffer[Index + 1] = '\0';
				break;
			}
		}

		return Buffer;
	}

	return nullptr;
}

const char* GetCacheDirectory()
{
	static char Buffer[1024] = { 0 };
	if (Buffer[0] == 0)
	{
		GetTempPathA(sizeof(Buffer), Buffer);
	}

	return Buffer;
}

int main(int argc, const char* argv[])
{
	return RunTests(argc, argv);
}
#endif