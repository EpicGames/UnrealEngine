// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_COREUOBJECT

#include "TestCommon/CoreUObjectUtilities.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/GCObject.h"

#if WITH_ENGINE
#if UE_LLT_USE_PLATFORM_FILE_STUB
#include "UObject/LinkerLoad.h"
#endif // UE_LLT_USE_PLATFORM_FILE_STUB
#if UE_LLT_WITH_MOCK_ENGINE_DEFAULTS
#include "Materials/Material.h"
#endif // UE_LLT_WITH_MOCK_ENGINE_DEFAULTS
#include "Styling/UMGCoreStyle.h"
#endif //WITH_ENGINE


#if WITH_ENGINE && UE_LLT_USE_PLATFORM_FILE_STUB
namespace
{
	const TCHAR* const KnownEngineMissingPackages[] = {
		TEXT("/Engine/EngineResources/DefaultTexture"),
		TEXT("/Engine/EngineResources/DefaultTextureCube"),
		TEXT("/Engine/EngineResources/DefaultVolumeTexture"),
		TEXT("/Engine/EngineFonts/RobotoDistanceField"),
		TEXT("/Engine/EngineMaterials/DefaultTextMaterialOpaque"),
		TEXT("/Engine/EngineDamageTypes/DmgTypeBP_Environmental"),
		TEXT("/Engine/EngineSky/VolumetricClouds/m_SimpleVolumetricCloud_Inst"),
		TEXT("/Engine/EngineMeshes/Sphere"),
		TEXT("/Engine/EngineResources/WhiteSquareTexture"),
		TEXT("/Engine/EngineResources/GradientTexture0"),
		TEXT("/Engine/EngineResources/Black"),
		TEXT("/Engine/EngineDebugMaterials/VolumeToRender"),
		TEXT("/Engine/EngineDebugMaterials/M_VolumeRenderSphereTracePP"),
		TEXT("/Engine/EngineFonts/Roboto"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent_OneSided"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Opaque"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Opaque_OneSided"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked"),
		TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked_OneSided")
	};
}
#endif // WITH_ENGINE && UE_LLT_USE_PLATFORM_FILE_STUB

void InitCoreUObject()
{
	IPackageResourceManager::Initialize();

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::FileSystemReady);

	if (!GetTransientPackage())
	{
		FModuleManager::Get().AddExtraBinarySearchPaths();
		FConfigCacheIni::InitializeConfigSystem();
		FPlatformFileManager::Get().InitializeNewAsyncIO();

#if WITH_ENGINE && UE_LLT_WITH_MOCK_ENGINE_DEFAULTS
		GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("AIControllerClassName"), TEXT("/Script/AIModule.AIController"), GEngineIni);
		GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultMaterialName"), TEXT("/Engine/Transient.MockDefaultMaterial"), GEngineIni);
		GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultLightFunctionMaterialName"), TEXT("/Engine/Transient.MockDefaultMaterial"), GEngineIni);
		GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultDeferredDecalMaterialName"), TEXT("/Engine/Transient.MockDefaultMaterial"), GEngineIni);
		GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultPostProcessMaterialName"), TEXT("/Engine/Transient.MockDefaultMaterial"), GEngineIni);
#endif // WITH_ENGINE && UE_LLT_WITH_MOCK_ENGINE_DEFAULTS

		FModuleManager::Get().LoadModule(TEXT("CoreUObject"));
		FCoreDelegates::OnInit.Broadcast();

#if WITH_ENGINE
		FUMGCoreStyle::ResetToDefault();
	#if UE_LLT_WITH_MOCK_ENGINE_DEFAULTS
		UMaterial* MockMaterial = NewObject<UMaterial>(GetTransientPackage(), UMaterial::StaticClass(), TEXT("MockDefaultMaterial"), RF_Transient | RF_MarkAsRootSet);
	#endif // UE_LLT_WITH_MOCK_ENGINE_DEFAULTS

	#if UE_LLT_USE_PLATFORM_FILE_STUB
		for (const TCHAR* MissingPackage : KnownEngineMissingPackages)
		{
			FLinkerLoad::AddKnownMissingPackage(FName(MissingPackage));
		}
	#endif // UE_LLT_USE_PLATFORM_FILE_STUB
#endif // WITH_ENGINE

		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::PreObjectSystemReady);

		ProcessNewlyLoadedUObjects();

		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::ObjectSystemReady);
	}
	FGCObject::StaticInit();
	if (GUObjectArray.IsOpenForDisregardForGC())
	{
		GUObjectArray.CloseDisregardForGC();
	}
}

void CleanupCoreUObject()
{
	IPackageResourceManager::Shutdown();

	FCoreDelegates::OnPreExit.Broadcast();
	FCoreDelegates::OnExit.Broadcast();
}

#endif // WITH_COREUOBJECT