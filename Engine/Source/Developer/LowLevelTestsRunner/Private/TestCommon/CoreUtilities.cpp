// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogSuppressionInterface.h"
#include "Misc/CommandLine.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/QueuedThreadPool.h"

#include "TestCommon/CoreUtilities.h"

#if WITH_APPLICATION_CORE
#include "HAL/PlatformApplicationMisc.h"
#endif

void InitCommandLine()
{
}

void CleanupCommandLine()
{
}

void InitGWarn()
{
	if (GWarn == nullptr)
	{
#if WITH_APPLICATION_CORE
		GWarn = FPlatformApplicationMisc::GetFeedbackContext();
#else
		GWarn = FPlatformOutputDevices::GetFeedbackContext();
#endif
	}
}

void InitIOThreadPool(bool MultiThreaded, int32 StackSize)
{
	GIOThreadPool = FQueuedThreadPool::Allocate();
	int32 NumThreadsInIOThreadPool = (MultiThreaded && FPlatformProcess::SupportsMultithreading()) ? FPlatformMisc::NumberOfIOWorkerThreadsToSpawn() : 1;
	verify(GIOThreadPool->Create(NumThreadsInIOThreadPool, StackSize, TPri_AboveNormal, TEXT("IOThreadPool")));
}

void InitThreadPool(bool MultiThreaded, int32 StackSize)
{
	GThreadPool = FQueuedThreadPool::Allocate();
	int32 NumThreadsInThreadPool = (MultiThreaded && FPlatformProcess::SupportsMultithreading()) ? FPlatformMisc::NumberOfWorkerThreadsToSpawn() : 1;
	verify(GThreadPool->Create(NumThreadsInThreadPool, StackSize, TPri_SlightlyBelowNormal, TEXT("ThreadPool")));
}

void InitBackgroundPriorityThreadPool(bool MultiThreaded, int32 StackSize)
{
	GBackgroundPriorityThreadPool = FQueuedThreadPool::Allocate();
	int32 NumThreadsInBgPriorityThreadPool = (MultiThreaded && FPlatformProcess::SupportsMultithreading()) ? 2 : 1;
	verify(GBackgroundPriorityThreadPool->Create(NumThreadsInBgPriorityThreadPool, StackSize, TPri_Lowest, TEXT("BackgroundThreadPool")));
}

void CleanupThreadPool()
{
	if (GThreadPool != nullptr)
	{
		GThreadPool->Destroy();
	}
}

void CleanupIOThreadPool()
{
	if (GIOThreadPool != nullptr)
	{
		GIOThreadPool->Destroy();
	}
}

void CleanupBackgroundPriorityThreadPool()
{
	if (GBackgroundPriorityThreadPool != nullptr)
	{
		GBackgroundPriorityThreadPool->Destroy();
	}
}

void CleanupAllThreadPools()
{
	CleanupThreadPool();
	CleanupIOThreadPool();
	CleanupBackgroundPriorityThreadPool();
}

void InitAllThreadPools(bool MultiThreaded)
{
	InitIOThreadPool(MultiThreaded);
	InitThreadPool(MultiThreaded);
	InitBackgroundPriorityThreadPool(MultiThreaded);
}

void InitTaskGraph(bool MultiThreaded, ENamedThreads::Type ThreadToAttach)
{
	FTaskGraphInterface::Startup(MultiThreaded ? FPlatformMisc::NumberOfWorkerThreadsToSpawn() : 1);
	FTaskGraphInterface::Get().AttachToThread(ThreadToAttach);

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::TaskGraphSystemReady);
}

void CleanupTaskGraph()
{
	FTaskGraphInterface::Shutdown();
}

void InitTaskGraphAndDependencies(bool MultiThreaded)
{
	InitTaskGraph(MultiThreaded);
	InitAllThreadPools(MultiThreaded);
}

void CleanupTaskGraphAndDependencies()
{
	CleanupAllThreadPools();
	CleanupTaskGraph();
}

void CleanupPlatform()
{
	FPlatformMisc::PlatformTearDown();
}