// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Misc/QueuedThreadPool.h"

#if WITH_COREUOBJECT
#include "UObject/PackageResourceManager.h"
#endif

#ifdef DERIVEDDATACACHE_API
#include "DerivedDataBuild.h"
#include "DerivedDataCache.h"
#endif

void InitEditorThreadPools()
{
	GLargeThreadPool = new FQueuedLowLevelThreadPool();

	// GThreadPool will schedule on the LargeThreadPool but limit max concurrency to the given number.
	GThreadPool = new FQueuedThreadPoolWrapper(GLargeThreadPool, FPlatformMisc::NumberOfWorkerThreadsToSpawn());
}

void InitDerivedDataCache()
{
#if WITH_EDITORONLY_DATA && defined(DERIVEDDATACACHE_API)
	if (!FPlatformProperties::RequiresCookedData())
	{
		// Ensure that DDC is initialized from the game thread.
		UE::DerivedData::GetCache();
		UE::DerivedData::GetBuild();
		GetDerivedDataCacheRef();
	}
#endif // WITH_EDITORONLY_DATA && defined(DERIVEDDATACACHE_API)
}

void InitForWithEditorOnlyData()
{
#if WITH_COREUOBJECT
	//Initialize the PackageResourceManager, which is needed to load any (non-script) Packages. It is first used in ProcessNewlyLoadedObjects (due to the loading of asset references in Class Default Objects)
	// It has to be intialized after the AssetRegistryModule; the editor implementations of PackageResourceManager relies on it
	IPackageResourceManager::Initialize();
#endif // WITH_COREUOBJECT
#if WITH_EDITOR
	// Initialize the BulkDataRegistry, which registers BulkData structs loaded from Packages for later building. It uses the same lifetime as IPackageResourceManager
	IBulkDataRegistry::Initialize();
#endif // WITH_EDITOR
}

void InitEditor()
{
#if WITH_EDITOR
	FModuleManager::Get().LoadModuleChecked("UnrealEd");

	GIsEditor = true;
	GEngine = GEditor = NewObject<UEditorEngine>(GetTransientPackage(), UEditorEngine::StaticClass());

	GEngine->ParseCommandline();
	GEditor->InitEditor(&GEngineLoop);
#endif // #if WITH_EDITOR
}

void InitSlate()
{
	FSlateApplication::Create();

	TSharedPtr<FSlateRenderer> SlateRenderer = FModuleManager::Get().LoadModuleChecked<ISlateNullRendererModule>("SlateNullRenderer").CreateSlateNullRenderer();
	TSharedRef<FSlateRenderer> SlateRendererSharedRef = SlateRenderer.ToSharedRef();

	// If Slate is being used, initialize the renderer after RHIInit
	FSlateApplication& CurrentSlateApp = FSlateApplication::Get();
	CurrentSlateApp.InitializeRenderer(SlateRendererSharedRef);
}

#endif // WITH_EDITOR
