// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ENGINE

#include "TestCommon/EngineUtilities.h"

#include "CoreMinimal.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardRepresentation.h"
#include "Modules/ModuleManager.h"
#include "RenderUtils.h"
#include "ShaderParameterMetadata.h"

#if WITH_COREUOBJECT
	#include "Internationalization/EnginePackageLocalizationCache.h"
	#include "Internationalization/PackageLocalizationManager.h"
	#include "Templates/SharedPointer.h"
#endif

struct GlobalEngineInitialization
{
	GlobalEngineInitialization()
	{
		// Any initialization to be triggered before other global objects
	}
} GEngineInitializationLowLevelTests;

void InitAsyncQueues()
{
	check(!GDistanceFieldAsyncQueue);
	GDistanceFieldAsyncQueue = new FDistanceFieldAsyncQueue();
}

void InitRendering()
{
	FShaderParametersMetadataRegistration::CommitAll();
	FShaderTypeRegistration::CommitAll();
	FShaderParametersMetadata::InitializeAllUniformBufferStructs();

	{
		// Initialize the RHI.
		const bool bHasEditorToken = false;
		RHIInit(bHasEditorToken);
	}

	{
		// One-time initialization of global variables based on engine configuration.
		RenderUtilsInit();
	}
}

void InitEngine()
{
	FModuleManager::Get().LoadModule(TEXT("Engine"));
	FModuleManager::Get().LoadModule(TEXT("RenderCore"));

#if WITH_COREUOBJECT
	FPackageLocalizationManager::Get().InitializeFromLazyCallback([](FPackageLocalizationManager& InPackageLocalizationManager)
	{
		InPackageLocalizationManager.InitializeFromCache(MakeShareable(new FEnginePackageLocalizationCache()));
	});
#endif
}

void CleanupEngine()
{
}

#endif // WITH_ENGINE
