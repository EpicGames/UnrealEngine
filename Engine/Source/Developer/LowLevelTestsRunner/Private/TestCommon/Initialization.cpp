// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestCommon/Initialization.h"
#include "TestCommon/CoreUtilities.h"
#include "TestCommon/CoreUObjectUtilities.h"
#include "TestCommon/EngineUtilities.h"

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/Paths.h"
#include "Sanitizer/RaceDetector.h"

static IPlatformFile* DefaultPlatformFile;

void InitAllThreadPoolsEditorEx(bool MultiThreaded)
{
#if WITH_EDITOR
	InitEditorThreadPools();
#endif // WITH_EDITOR
	InitAllThreadPools(MultiThreaded);
}

void InitStats()
{
#if STATS
	FThreadStats::StartThread();
#endif // #if STATS

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::StatSystemReady);
}

void UsePlatformFileStubIfRequired()
{
#if UE_LLT_USE_PLATFORM_FILE_STUB
	if (IPlatformFile* WrapperFile = FPlatformFileManager::Get().GetPlatformFile(TEXT("LowLevelTestsRunner")))
	{
		IPlatformFile* CurrentPlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
		WrapperFile->Initialize(CurrentPlatformFile, TEXT(""));
		FPlatformFileManager::Get().SetPlatformFile(*WrapperFile);
	}
#endif // UE_LLT_USE_PLATFORM_FILE_STUB
}

void SaveDefaultPlatformFile()
{
	DefaultPlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
}

void UseDefaultPlatformFile()
{
	FPlatformFileManager::Get().SetPlatformFile(*DefaultPlatformFile);
}

void SetProjectNameAndDirectory()
{
	// Use target name instead of project file being passed in as this will be more accruate to finding the program's config
	// and project files. Resolves issues with UE_PROJECT_NAME being set to the parent project instead of the program's project.
	// This will likely break if we ever have one LLT project file have mutliple targets eg: LyraGameEOS vs LyraGame
#ifdef UE_TARGET_NAME
	FCString::Strncpy(GInternalProjectName, TEXT(UE_STRINGIZE(UE_TARGET_NAME)), UE_ARRAY_COUNT(GInternalProjectName));
#endif

	// There may be cases on some platforms that you need to verify files are in the filesystem, 
	// and this call was needed to correctly initalize the system.
	FPaths::ProjectDir();

	FString ProjectFileOrName;
	FString ProjectDirOverride;

	bool bIsProjectNamePassed = false;

	FParse::Value(FCommandLine::Get(), TEXT("-project="), ProjectFileOrName);
	if (!ProjectFileOrName.IsEmpty())
	{
		bIsProjectNamePassed = !ProjectFileOrName.EndsWith(TEXT(".uproject"));
	}

	FParse::Value(FCommandLine::Get(), TEXT("-projectdir="), ProjectDirOverride);
	if (!ProjectDirOverride.IsEmpty())
	{
		if (!ProjectDirOverride.EndsWith(TEXT("/")))
		{
			ProjectDirOverride.Append(TEXT("/"));
		}
	}

	if (!bIsProjectNamePassed && ProjectDirOverride.IsEmpty() && !ProjectFileOrName.IsEmpty())
	{
		ProjectDirOverride = FPaths::GetPath(ProjectFileOrName);
	}
	
	if (!ProjectDirOverride.IsEmpty())
	{
		FPaths::NormalizeDirectoryName(ProjectDirOverride);
		if (!ProjectDirOverride.EndsWith(TEXT("/")))
		{
			ProjectDirOverride.Append(TEXT("/"));
		}
		FGenericPlatformMisc::SetOverrideProjectDir(ProjectDirOverride);
	}
}

void InitAll(bool bAllowLogging, bool bMultithreaded)
{
#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::Initialize();
	UE::Sanitizer::RaceDetector::ToggleRaceDetection(true);
#endif
	SaveDefaultPlatformFile();
	UsePlatformFileStubIfRequired();
	InitAllThreadPools(bMultithreaded);
#if WITH_ENGINE
	InitAsyncQueues();
#endif // WITH_ENGINE
	InitTaskGraph();
#if WITH_ENGINE
	InitGWarn();
	InitEngine();
#endif // WITH_ENGINE
#if WITH_EDITOR
	InitDerivedDataCache();
	InitSlate();
	InitForWithEditorOnlyData();
	InitEditor();
#endif // WITH_EDITOR
#if WITH_COREUOBJECT
	InitCoreUObject();
#endif
	GIsRunning = true;
}

void CleanupLocalization()
{
	FTextLocalizationManager::TearDown();
	FInternationalization::TearDown();
}

void CleanupAll()
{
#if WITH_ENGINE
	CleanupEngine();
#endif
#if WITH_COREUOBJECT
	CleanupCoreUObject();
#endif
	CleanupAllThreadPools();
	CleanupTaskGraph();
	CleanupLocalization();

#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::Shutdown();
#endif
}