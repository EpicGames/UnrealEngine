// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"

#include <catch2/catch_test_case_info.hpp>
#include <catch2/catch_timer.hpp>
#include <catch2/reporters/catch_reporter_registrars.hpp>
#include <catch2/reporters/catch_reporter_streaming_base.hpp>

#include <catch2/internal/catch_jsonwriter.hpp>
#include <catch2/internal/catch_stringref.hpp>

#include <sstream>
#include <stack>
#include <string>

using namespace Catch;

namespace UE::LowLevelTests
{
	class FTestDataReporter : public StreamingReporterBase
	{
	private:
		std::string m_ReportFilePath;
	public:
		using StreamingReporterBase::StreamingReporterBase;

		FTestDataReporter(ReporterConfig&& Config)
			: StreamingReporterBase(CATCH_MOVE(Config))
		{
			m_preferences.shouldRedirectStdOut = true;
			m_preferences.shouldReportAllAssertions = true;
			m_preferences.shouldReportAllAssertionStarts = false;

			// Initialize JSON writer with the stringstream
			m_objectWriters.emplace(m_stream);
			m_writers.emplace(Writer::Object);
		}

		~FTestDataReporter() {
			m_stream << std::endl << std::flush;
			assert(m_writers.empty());
		}

		static std::string getDescription()
		{
			return "Low Level Tests TestData JSON Reporter";
		}

		void testRunStarting(TestRunInfo const& TestInfo) override
		{
			StreamingReporterBase::testRunStarting(TestInfo);

			Succeeded = 0;
			SucceededWithWarnings = 0;
			Failed = 0;
			NotRun = 0;
			InProcess = 0;
			TotalDuration = 0.0f;

			ReportCreatedOn = FDateTime::UtcNow();

			FString OSMajorVersionString, OSSubVersionString;
			FPlatformMisc::GetOSVersions(OSMajorVersionString, OSSubVersionString);

			FString OSVersionString = OSMajorVersionString + TEXT(" ") + OSSubVersionString;

			// Write Devices array
			startArray("Devices");
			auto& DeviceObj = startObject();
			DeviceObj.write("DeviceName").write(TCHAR_TO_UTF8(*FString(FPlatformProcess::ComputerName())));
			DeviceObj.write("InstanceName").write(TCHAR_TO_UTF8(*FApp::GetInstanceName()));
			DeviceObj.write("Platform").write(TCHAR_TO_UTF8(*FString(FPlatformProperties::PlatformName())));
			DeviceObj.write("SessionId").write(TCHAR_TO_UTF8(*LexToString(FApp::GetSessionId())));
			DeviceObj.write("OSVersion").write(TCHAR_TO_UTF8(*OSVersionString));
			DeviceObj.write("Mode").write(TCHAR_TO_UTF8(*FString(FPlatformMisc::GetDefaultDeviceProfileName())));
			DeviceObj.write("CPU").write(TCHAR_TO_UTF8(*FPlatformMisc::GetCPUBrand().TrimStart()));
#if !PLATFORM_ANDROID
			DeviceObj.write("GPU").write(TCHAR_TO_UTF8(*FPlatformMisc::GetPrimaryGPUBrand()));
#endif
			DeviceObj.write("RAMInGB").write(FPlatformMemory::GetPhysicalGBRam());
			DeviceObj.write("RHI").write(TCHAR_TO_UTF8(*FApp::GetGraphicsRHI()));
			endObject();
			endArray();

			// Write ReportCreatedOn
			m_objectWriters.top().write("ReportCreatedOn").write(TCHAR_TO_UTF8(*ReportCreatedOn.ToIso8601()));

			// Start Tests array
			startArray("Tests");
		}

		void testRunEnded(TestRunStats const& TestRunStats) override
		{
			// End Tests array
			endArray();

			// Write summary statistics
			auto& rootObj = m_objectWriters.top();
			rootObj.write("Succeeded").write(Succeeded);
			rootObj.write("SucceededWithWarnings").write(SucceededWithWarnings);
			rootObj.write("Failed").write(Failed);
			rootObj.write("NotRun").write(NotRun);
			rootObj.write("InProcess").write(InProcess);
			rootObj.write("TotalDuration").write(TotalDuration);
			rootObj.write("ComparisonExported").write(false);
			rootObj.write("ComparisonExportDirectory").write("");

			// End root object
			endObject();
		}

		void testCaseStarting(TestCaseInfo const& TestCaseInfo) override
		{
			Timer.start();

			CurrentTestErrors = 0;
			CurrentTestWarnings = 0;
			CurrentTestStartTime = FDateTime::UtcNow();
			CurrentTestEntries.Empty();

			// Start test object
			auto& testObj = startObject();

			// Write TestDisplayName and FullTestPath
			testObj.write("TestDisplayName").write(TestCaseInfo.name.c_str());
			testObj.write("FullTestPath").write(TestCaseInfo.name.c_str());

			// Write Tags array
			auto& tagsArray = startArray("Tags");
			FString TestTags(TestCaseInfo.tagsAsString().c_str());
			TArray<FString> TagArray;
			TestTags.Replace(TEXT("]"), TEXT("[")).ParseIntoArray(TagArray, TEXT("["));
			for (const FString& TagStr : TagArray)
			{
				if (!TagStr.IsEmpty())
				{
					tagsArray.write(TCHAR_TO_UTF8(*TagStr));
				}
			}
			endArray();

			InProcess++;
		}

		void testCaseEnded(TestCaseStats const& TestCaseStats) override
		{
			double Duration = Timer.getElapsedSeconds();
			TotalDuration += static_cast<float>(Duration);

			InProcess--;

			// Determine final state
			FString State;
			if (TestCaseStats.totals.assertions.allOk())
			{
				if (CurrentTestWarnings > 0)
				{
					State = TEXT("Success");
					SucceededWithWarnings++;
				}
				else
				{
					State = TEXT("Success");
					Succeeded++;
				}
			}
			else
			{
				State = TEXT("Fail");
				Failed++;
			}

			// Write remaining test fields
			auto& testObj = m_objectWriters.top();
			testObj.write("State").write(TCHAR_TO_UTF8(*State));

			// Write empty DeviceInstance array
			startArray("DeviceInstance");
			endArray();

			testObj.write("Duration").write(static_cast<float>(Duration));
			testObj.write("DateTime").write(TCHAR_TO_UTF8(*CurrentTestStartTime.ToIso8601()));

			// Write Entries array
			auto& entriesArray = startArray("Entries");
			for (const FBufferedEntry& Entry : CurrentTestEntries)
			{
				auto& entryObj = startObject();

				// Write Event object
				{
					auto& eventObj = startObject("Event");
					eventObj.write("Type").write(TCHAR_TO_UTF8(*Entry.Type));
					eventObj.write("Message").write(TCHAR_TO_UTF8(*Entry.Message));
					eventObj.write("Context").write(TCHAR_TO_UTF8(*Entry.Context));
					eventObj.write("Artifact").write(TCHAR_TO_UTF8(*Entry.Artifact));
					endObject();
				}

				entryObj.write("Filename").write(TCHAR_TO_UTF8(*Entry.Filename));
				entryObj.write("LineNumber").write(Entry.LineNumber);
				entryObj.write("Timestamp").write(TCHAR_TO_UTF8(*Entry.Timestamp.ToIso8601()));

				endObject();
			}
			endArray();

			testObj.write("Warnings").write(CurrentTestWarnings);
			testObj.write("Errors").write(CurrentTestErrors);

			// Write empty Artifacts array
			startArray("Artifacts");
			endArray();

			// End test object
			endObject();
		}

		void skipTest(TestCaseInfo const& TestInfo) override
		{
			// Start test object
			auto& testObj = startObject();

			// Write test fields
			testObj.write("TestDisplayName").write(TestInfo.name.c_str());
			testObj.write("FullTestPath").write(TestInfo.name.c_str());

			// Write Tags array
			{
				auto& tagsArray = startArray("Tags");
				FString TestTags(TestInfo.tagsAsString().c_str());
				TArray<FString> TagArray;
				TestTags.Replace(TEXT("]"), TEXT("[")).ParseIntoArray(TagArray, TEXT("["));
				for (const FString& TagStr : TagArray)
				{
					if (!TagStr.IsEmpty())
					{
						tagsArray.write(TCHAR_TO_UTF8(*TagStr));
					}
				}
				endArray();
			}

			testObj.write("State").write("Skipped");

			// Write empty DeviceInstance array
			startArray("DeviceInstance");
			endArray();

			testObj.write("Duration").write(0.0f);
			testObj.write("DateTime").write(TCHAR_TO_UTF8(*FDateTime::UtcNow().ToIso8601()));

			// Write empty Entries array
			startArray("Entries");
			endArray();

			testObj.write("Warnings").write(0);
			testObj.write("Errors").write(0);

			// Write empty Artifacts array
			startArray("Artifacts");
			endArray();

			// End test object
			endObject();
		}

		void assertionEnded(AssertionStats const& Stats) override
		{
			AssertionResult const& Result = Stats.assertionResult;

			if (!Result.isOk())
			{
				FBufferedEntry Entry;

				switch (Result.getResultType())
				{
				case ResultWas::ThrewException:
				case ResultWas::FatalErrorCondition:
				case ResultWas::ExplicitFailure:
				case ResultWas::ExpressionFailed:
				case ResultWas::DidntThrowException:
					Entry.Type = TEXT("Error");
					CurrentTestErrors++;
					break;
				case ResultWas::Warning:
					Entry.Type = TEXT("Warning");
					CurrentTestWarnings++;
					break;
				case ResultWas::ExplicitSkip:
					Entry.Type = TEXT("Skipped");
					NotRun++;
					break;
				default:
					Entry.Type = TEXT("Error");
					CurrentTestErrors++;
					break;
				}

				// Build the message
				TAnsiStringBuilder<1024> MessageBuilder;
				if (Stats.totals.assertions.total() > 0)
				{
					MessageBuilder << "FAILED:\n";
					if (Result.hasExpression())
					{
						MessageBuilder << "  " << Result.getExpressionInMacro().c_str() << "\n";
					}
					if (Result.hasExpandedExpression())
					{
						MessageBuilder << "with expansion:\n" << Result.getExpandedExpression().c_str() << "\n";
					}
				}

				if (!Result.getMessage().empty())
				{
					MessageBuilder << Result.getMessage().data() << '\n';
				}

				for (auto const& Msg : Stats.infoMessages)
				{
					if (Msg.type == ResultWas::Info)
					{
						MessageBuilder << Msg.message.data() << '\n';
					}
				}

				MessageBuilder << "at " << Result.getSourceInfo().file << '(' << (uint64)Result.getSourceInfo().line << ')';

				Entry.Message = FString(UTF8_TO_TCHAR(*MessageBuilder));
				Entry.Context = TEXT("");
				Entry.Artifact = TEXT("");
				Entry.Filename = FString(Result.getSourceInfo().file);
				Entry.LineNumber = static_cast<int32>(Result.getSourceInfo().line);
				Entry.Timestamp = FDateTime::UtcNow();

				CurrentTestEntries.Add(Entry);
			}
		}

	private:
		enum class Writer {
			Object,
			Array
		};

		JsonArrayWriter& startArray() {
			m_arrayWriters.emplace(m_arrayWriters.top().writeArray());
			m_writers.emplace(Writer::Array);
			return m_arrayWriters.top();
		}
		JsonArrayWriter& startArray(StringRef key) {
			m_arrayWriters.emplace(
				m_objectWriters.top().write(key).writeArray());
			m_writers.emplace(Writer::Array);
			return m_arrayWriters.top();
		}

		JsonObjectWriter& startObject() {
			m_objectWriters.emplace(m_arrayWriters.top().writeObject());
			m_writers.emplace(Writer::Object);
			return m_objectWriters.top();
		}
		JsonObjectWriter& startObject(StringRef key) {
			m_objectWriters.emplace(
				m_objectWriters.top().write(key).writeObject());
			m_writers.emplace(Writer::Object);
			return m_objectWriters.top();
		}

		void endObject() {
			assert(isInside(Writer::Object));
			m_objectWriters.pop();
			m_writers.pop();
		}
		void endArray() {
			assert(isInside(Writer::Array));
			m_arrayWriters.pop();
			m_writers.pop();
		}

		bool isInside(Writer writer) {
			return !m_writers.empty() && m_writers.top() == writer;
		}

		// Invariant:
		// When m_writers is not empty and its top element is
		// - Writer::Object, then m_objectWriters is not be empty
		// - Writer::Array,  then m_arrayWriters shall not be empty
		std::stack<JsonObjectWriter> m_objectWriters{};
		std::stack<JsonArrayWriter> m_arrayWriters{};
		std::stack<Writer> m_writers{};

		// Buffered entry structure for collecting entries during test execution
		struct FBufferedEntry
		{
			FString Type;
			FString Message;
			FString Context;
			FString Artifact;
			FString Filename;
			int32 LineNumber;
			FDateTime Timestamp;
		};

		TArray<FBufferedEntry> CurrentTestEntries;
		int32 CurrentTestErrors;
		int32 CurrentTestWarnings;
		FDateTime CurrentTestStartTime;
		Timer Timer;

		FDateTime ReportCreatedOn;
		int32 Succeeded;
		int32 SucceededWithWarnings;
		int32 Failed;
		int32 NotRun;
		int32 InProcess;
		float TotalDuration;
	};

	CATCH_REGISTER_REPORTER("testdata", FTestDataReporter)

}