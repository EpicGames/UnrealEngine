// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Modules/ModuleManager.h"

// Allow UBT to name my application
#if !defined(UE_PROJECT_NAME)
TCHAR GInternalProjectName[64] = TEXT("LowLevelTests");
#else
TCHAR GInternalProjectName[64] = TEXT( UE_STRINGIZE(UE_PROJECT_NAME) );
#endif 
IMPLEMENT_FOREIGN_ENGINE_DIR()

// Typical defined by TargetRules but LowLevelTestRunner is not setup correctly
// Should revist this in the future
#ifndef IMPLEMENT_ENCRYPTION_KEY_REGISTRATION
	#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#endif
#ifndef IMPLEMENT_SIGNING_KEY_REGISTRATION
	#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
#endif

#if defined(UE_TARGET_NAME)
IMPLEMENT_TARGET_NAME_REGISTRATION()
#endif

// Debug visualizers and new operator overloads
PER_MODULE_BOILERPLATE