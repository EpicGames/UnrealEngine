// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

std::ostream& operator<<(std::ostream& Stream, const TCHAR* Value)
{
	return Stream << FUtf8StringView(FTCHARToUTF8(Value));
}

std::ostream& operator<<(std::ostream& Stream, const FString& Value)
{
	return Stream << FUtf8StringView(FTCHARToUTF8(Value));
}

std::ostream& operator<<(std::ostream& Stream, const FAnsiStringView& Value)
{
	return Stream.write(Value.GetData(), Value.Len());
}

std::ostream& operator<<(std::ostream& Stream, const FWideStringView& Value)
{
	return Stream << FUtf8StringView(FTCHARToUTF8(Value));
}

std::ostream& operator<<(std::ostream& Stream, const FUtf8StringView& Value)
{
	return Stream.write((const char*)Value.GetData(), Value.Len());
}

std::ostream& operator<<(std::ostream& Stream, const FAnsiStringBuilderBase& Value)
{
	return Stream.write(Value.GetData(), Value.Len());
}

std::ostream& operator<<(std::ostream& Stream, const FWideStringBuilderBase& Value)
{
	return Stream << FUtf8StringView(FTCHARToUTF8(Value));
}

std::ostream& operator<<(std::ostream& Stream, const FUtf8StringBuilderBase& Value)
{
	return Stream << Value.ToView();
}
