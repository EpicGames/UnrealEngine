// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"
#include "TestRunnerPrivate.h"

#include <catch2/catch_test_case_info.hpp>
#include <catch2/reporters/catch_reporter_event_listener.hpp>
#include <catch2/reporters/catch_reporter_registrars.hpp>
#include <catch2/reporters/catch_reporter_streaming_base.hpp>

#include <thread>
#include <chrono>
#include <string>

namespace UE::LowLevelTests
{

DEFINE_LOG_CATEGORY_STATIC(LogLowLevelTests, Log, VeryVerbose);

class FEventListener final : public Catch::EventListenerBase
{
private:
	FDateTime TestStarted;
	std::thread* TimeoutThread = nullptr;
	std::string CurrentTest;

	void logMessageOnTimeout()
	{
		std::string CurrentTestLocal;
		if (ITestRunner* TestRunner = ITestRunner::Get())
		{
			while (true)
			{
				if (CurrentTestLocal != CurrentTest)
				{
					CurrentTestLocal = CurrentTest;
					std::this_thread::sleep_for(std::chrono::minutes(TestRunner->GetTimeoutMinutes()));
					UE_CLOGF(CurrentTestLocal == CurrentTest, LogLowLevelTests, Error,
						"Timeout detected: Test case \"%s\" has been running for more than %d minute(s)", CurrentTest.c_str(), TestRunner->GetTimeoutMinutes());
					
				}
				else // test still running after timeout reached, prevent CPU spinning
				{
					std::this_thread::sleep_for(std::chrono::seconds(1));
				}
			}
		}
	}

public:
	using EventListenerBase::EventListenerBase;

	void sectionStarting(const Catch::SectionInfo& SectionInfo) final
	{
		if (ITestRunner* TestRunner = ITestRunner::Get())
		{
			if (SectionInfo.name != CurrentTest)
			{
				UE_CLOG(TestRunner->IsDebugMode(), LogLowLevelTests, Display,
					TEXT("running section \"%hs\" (line %" SIZE_T_FMT ")"),
					SectionInfo.name.c_str(),
					SIZE_T(SectionInfo.lineInfo.line));
			}
		}
	}

	void testCaseStarting(const Catch::TestCaseInfo& TestInfo) final
	{
		if (ITestRunner* TestRunner = ITestRunner::Get())
		{
			CurrentTest = TestInfo.name;

			if (TimeoutThread == nullptr && TestRunner->GetTimeoutMinutes() > 0)
			{
				TimeoutThread = new std::thread(&FEventListener::logMessageOnTimeout, this);
			}

			TestStarted = FDateTime::Now();
			UE_CLOG(TestRunner->IsDebugMode(), LogLowLevelTests, Display,
				TEXT("Started test case \"%hs\" at %hs(%" SIZE_T_FMT ") with tags %hs"),
				TestInfo.name.c_str(),
				TestInfo.lineInfo.file,
				SIZE_T(TestInfo.lineInfo.line),
				TestInfo.tagsAsString().c_str());
		}
	}

	void testCaseEnded(const Catch::TestCaseStats& TestCaseStats) final
	{
		if (ITestRunner* TestRunner = ITestRunner::Get())
		{
			FTimespan Duration = FDateTime::Now() - TestStarted;
			
			TStringBuilder<64> DurationText;
			const int32 Hours = Duration.GetHours();
			const int32 Minutes = Duration.GetMinutes();
			const double Seconds = Duration.GetSeconds();
			if (Hours > 0)
			{
				DurationText << Hours << TEXTVIEW("h ");
			}
			if (Hours > 0 || Minutes > 0)
			{
				DurationText << Minutes << TEXTVIEW("m ");
			}
			if (Hours > 0 || Minutes > 0 || Seconds >= 1.0)
			{
				DurationText.Appendf(TEXT("%.3fs "), Seconds);
			}
			else
			{
				DurationText << Duration.GetFractionMilli() << TEXTVIEW("ms");
			}

			UE_CLOGF(TestRunner->IsDebugMode(), LogLowLevelTests, Display,
				"Test took %ls",
				*DurationText);

			UE_CLOG(TestRunner->IsDebugMode() && TestCaseStats.totals.testCases.failed > 0,
				LogLowLevelTests, Error, TEXT("Failed test case \"%hs\" at %hs(%" SIZE_T_FMT ")"),
				TestCaseStats.testInfo->name.c_str(),
				TestCaseStats.testInfo->lineInfo.file,
				SIZE_T(TestCaseStats.testInfo->lineInfo.line));
		}
	}

	void assertionEnded(const Catch::AssertionStats& AssertionStats) final
	{
		if (ITestRunner* TestRunner = ITestRunner::Get())
		{
			const bool bShouldLogAssertion = TestRunner->HasLogOutput() && !AssertionStats.assertionResult.succeeded();
			if (bShouldLogAssertion)
			{
				if (AssertionStats.assertionResult.hasExpandedExpression())
				{
					UE_LOG( LogLowLevelTests, Error, TEXT("Assertion \"%hs\" failed with \"%hs\" at %hs(%" SIZE_T_FMT ")"),
							AssertionStats.assertionResult.getExpression().c_str(),
							AssertionStats.assertionResult.getExpandedExpression().c_str(),
							AssertionStats.assertionResult.getSourceInfo().file,
							SIZE_T(AssertionStats.assertionResult.getSourceInfo().line));
				}
				else
				{
					UE_LOG( LogLowLevelTests, Error, TEXT("Assertion \"%hs\" failed at %hs(%" SIZE_T_FMT ")"),
							AssertionStats.assertionResult.getExpression().c_str(),
							AssertionStats.assertionResult.getSourceInfo().file,
							SIZE_T(AssertionStats.assertionResult.getSourceInfo().line));
				}

				for (const Catch::MessageInfo& Info : AssertionStats.infoMessages)
				{
					UE_LOG( LogLowLevelTests, Warning, TEXT("Info: \"%hs\" at %hs(%" SIZE_T_FMT ")"),
							Info.message.c_str(),
							Info.lineInfo.file,
							SIZE_T(Info.lineInfo.line));
				}
			}
		}
	}
};

CATCH_REGISTER_LISTENER(FEventListener);

} // UE::LowLevelTests
