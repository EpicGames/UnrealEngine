// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestRunner.h"
#include "TestRunnerPrivate.h"

#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformTLS.h"
#include "Logging/LogSuppressionInterface.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Sanitizer/RaceDetector.h"
#include "String/Find.h"
#include "String/LexFromString.h"
#include "Containers/UnrealString.h"
#include "TestCommon/CoreUtilities.h"
#include "TestRunnerOutputDeviceError.h"

#if WITH_APPLICATION_CORE
#include "HAL/PlatformApplicationMisc.h"
#endif

#include "Misc/CoreDelegates.h"
#include <catch2/catch_session.hpp>
#include <catch2/internal/catch_assertion_handler.hpp>

#include <iostream>

namespace UE::LowLevelTests
{
static ITestRunner* GTestRunner;

ITestRunner* ITestRunner::Get()
{
	return GTestRunner;
}

ITestRunner::ITestRunner()
{
	check(!GTestRunner);
	GTestRunner = this;
}

ITestRunner::~ITestRunner()
{
	check(GTestRunner == this);
	GTestRunner = nullptr;
}

class FTestRunner final : public ITestRunner
{
public:
	FTestRunner();

	void ParseCommandLine(TConstArrayView<const ANSICHAR*> Args);

	void SleepOnInit() const;

	void GlobalSetup();
	void GlobalTeardown() const;
	void Terminate() const;

	int32 RunCatchSession() const;

	bool HasGlobalSetup() const final { return bGlobalSetup; }
	bool HasLogOutput() const final { return bLogOutput || bDebugMode; }
	bool IsDebugMode() const final { return bDebugMode; }

	int32 GetTimeoutMinutes() const final { return TimeoutMinutes; }

private:
	TArray<const ANSICHAR*> CatchArgs;
	TStringBuilder<0> ExtraArgs;
	FTestRunnerOutputDeviceError ErrorOutputDevice;
	bool bGlobalSetup = true;
	bool bLogOutput = false;
	bool bDebugMode = false;
	bool bMultiThreaded = false;
	bool bWaitForInputToTerminate = false;
	bool bAttachToDebugger = false;
	int32 SleepOnInitSeconds = 0;
	int32 TimeoutMinutes = 0;
};

FTestRunner::FTestRunner()
{
	// Start setting up the Game Thread.
	GGameThreadId = FPlatformTLS::GetCurrentThreadId();
	GIsGameThreadIdInitialized = true;
}

void FTestRunner::ParseCommandLine(TConstArrayView<const ANSICHAR*> Args)
{
	bool bExtraArg = false;

	for (FAnsiStringView Arg : Args)
	{
		if (bExtraArg)
		{
			if (const int32 SpaceIndex = String::FindFirstChar(Arg, ' '); SpaceIndex != INDEX_NONE)
			{
				if (const int32 EqualIndex = String::FindFirstChar(Arg, '='); EqualIndex != INDEX_NONE && EqualIndex < SpaceIndex)
				{
					ExtraArgs.Append(Arg.Left(EqualIndex + 1));
					Arg.RightChopInline(EqualIndex + 1);
				}
				ExtraArgs.AppendChar('"').Append(Arg).AppendChar('"').AppendChar(' ');
			}
			else
			{
				ExtraArgs.Append(Arg).AppendChar(' ');
			}
		}
		else if (Arg == ANSITEXTVIEW("--extra-args"))
		{
			bExtraArg = true;
		}

		else if (Arg.StartsWith(ANSITEXTVIEW("--sleep=")))
		{
			LexFromString(SleepOnInitSeconds, WriteToString<16>(Arg.RightChop(8)).ToView());
		}
		else if (Arg.StartsWith(ANSITEXTVIEW("--timeout=")))
		{
			LexFromString(TimeoutMinutes, WriteToString<16>(Arg.RightChop(10)).ToView());
		}
		else if (Arg == ANSITEXTVIEW("--global-setup"))
		{
			bGlobalSetup = true;
		}
		else if (Arg == ANSITEXTVIEW("--no-global-setup"))
		{
			bGlobalSetup = false;
		}
		else if (Arg == ANSITEXTVIEW("--log"))
		{
			bLogOutput = true;
		}
		else if (Arg == ANSITEXTVIEW("--no-log"))
		{
			bLogOutput = false;
		}
		else if (Arg == ANSITEXTVIEW("--debug"))
		{
			bDebugMode = true;
		}
		else if (Arg == ANSITEXTVIEW("--mt"))
		{
			bMultiThreaded = true;
		}
		else if (Arg == ANSITEXTVIEW("--no-mt"))
		{
			bMultiThreaded = false;
		}
		else if (Arg == ANSITEXTVIEW("--wait"))
		{
			bWaitForInputToTerminate = true;
		}
		else if (Arg == ANSITEXTVIEW("--no-wait"))
		{
			bWaitForInputToTerminate = false;
		}
		else if (Arg == ANSITEXTVIEW("--attach-to-debugger"))
		{
			bAttachToDebugger = true;
		}
		else if (Arg == ANSITEXTVIEW("--waitfordebugger"))
		{
			bAttachToDebugger = true;
		}
		else if (Arg == ANSITEXTVIEW("--buildmachine"))
		{
			GIsBuildMachine = true;
		}
		else
		{
			CatchArgs.Add(Arg.GetData());
		}
	}

	if (!CatchArgs.Contains(ANSITEXTVIEW("--break")))
	{
		// Break in the debugger on failed assertions when attached.
		CatchArgs.Add("--break");
	}
}

void FTestRunner::SleepOnInit() const
{
	if (SleepOnInitSeconds)
	{
		// Sleep to allow sync with Gauntlet.
		FPlatformProcess::Sleep(SleepOnInitSeconds);
	}
}

void FTestRunner::GlobalSetup()
{
	if (bAttachToDebugger)
	{
		FPlatformMisc::LocalPrint(TEXT("Waiting for debugger..."));
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.1f);
		}
		UE_DEBUG_BREAK();
	}

	if (!bGlobalSetup)
	{
		return;
	}

	FCommandLine::Set(*ExtraArgs);

	// Finish setting up the Game Thread, which requires the command line.
	FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetMainGameMask());
	FPlatformProcess::SetupGameThread();

	// Always set up GError to handle FatalError, failed assertions, and crashes and other fatal errors.
#if WITH_APPLICATION_CORE
	GError = FPlatformApplicationMisc::GetErrorOutputDevice();
#else
	GError = FPlatformOutputDevices::GetError();
	ErrorOutputDevice.SetDeviceError(GError);
	GError = &ErrorOutputDevice;
#endif
	
	//forward unhandled `ensure` to catch to force tests to fail. test will continue to execute
	//this does bypass the error reporting, crash reporter and etc
	FCoreDelegates::OnHandleSystemEnsure.AddLambda([this]()
		{
			FString Error = GErrorHist;
			Catch::AssertionInfo info{ "", CATCH_INTERNAL_LINEINFO, "", Catch::ResultDisposition::Normal };
			Catch::AssertionReaction reaction;
			Catch::getResultCapture().handleMessage(info, Catch::ResultWas::ExplicitFailure, StringCast<ANSICHAR>(*Error).Get(), reaction);
		});

	// Set up GWarn to handle Error, Warning, Display; but only when log output is enabled.
#if WITH_APPLICATION_CORE
	GWarn = FPlatformApplicationMisc::GetFeedbackContext();
#else
	GWarn = FPlatformOutputDevices::GetFeedbackContext();
#endif
	if (bLogOutput || bDebugMode)
	{
		// Set up default output devices to handle Log, Verbose, VeryVerbose.
		FPlatformOutputDevices::SetupOutputDevices();
		FLogSuppressionInterface::Get().ProcessConfigAndCommandLine();
	}

	FTestDelegates::GetGlobalPlatformSetup().ExecuteIfBound();
	FTestDelegates::GetGlobalSetup().ExecuteIfBound();
}

void FTestRunner::GlobalTeardown() const
{
	if (!bGlobalSetup)
	{
		return;
	}
	
	//only set the GError back if it was replaced
	if (GError == &ErrorOutputDevice)
	{
		GError = ErrorOutputDevice.GetDeviceError();
	}

	FTestDelegates::GetGlobalTeardown().ExecuteIfBound();

	CleanupPlatform();
}

void FTestRunner::Terminate() const
{
#if PLATFORM_DESKTOP
	if (bWaitForInputToTerminate)
	{
		std::cout << "Press enter to exit..." << std::endl;
		std::cin.ignore();
	}
#endif
}

int32 FTestRunner::RunCatchSession() const
{
	return Catch::Session().run(CatchArgs.Num(), CatchArgs.GetData());
}

} // UE::LowLevelTests

int RunTests(int32 ArgC, const ANSICHAR* ArgV[])
{
#if USING_INSTRUMENTATION
	UE::Sanitizer::RaceDetector::Initialize();

	ON_SCOPE_EXIT
	{
		UE::Sanitizer::RaceDetector::Shutdown();
	};
#endif

	FTaskTagScope Scope(ETaskTag::EGameThread);

	UE::LowLevelTests::FTestRunner TestRunner;

	// Read command-line from file (if any). Some platforms do this earlier.
#ifndef PLATFORM_SKIP_ADDITIONAL_ARGS
	{
		int32 OverrideArgC = 0;
		const ANSICHAR** OverrideArgV = ReadAndAppendAdditionalArgs(GetProcessExecutablePath(), &OverrideArgC, ArgV, ArgC);
		if (OverrideArgV && OverrideArgC > 1)
		{
			ArgC = OverrideArgC;
			ArgV = OverrideArgV;
		}
	}
#endif

	TestRunner.ParseCommandLine(MakeArrayView(ArgV, ArgC));

	TestRunner.SleepOnInit();
	
	TestRunner.GlobalSetup();

	ON_SCOPE_EXIT
	{
		TestRunner.GlobalTeardown();
		TestRunner.Terminate();
		
		FModuleManager::Get().UnloadModulesAtShutdown();
		RequestEngineExit(TEXT("Exiting"));
		// TearDown GLog at the end so RequestEngineExit can properly log ExitCode
		if (GLog)
		{
			GLog->TearDown();
		}
	};

	int CatchReturn = TestRunner.RunCatchSession();
	return CatchReturn;
}
