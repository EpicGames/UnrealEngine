// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestRunnerOutputDeviceError.h"
#include "Containers/StringConv.h"
#include <catch2/internal/catch_assertion_handler.hpp>

namespace UE::LowLevelTests
{

//report error back to catch otherwise failed `check`'s don't fail the tests
//this method is not execute if the debugger is attached
void FTestRunnerOutputDeviceError::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	auto AnsiMessage = StringCast<ANSICHAR>(V);
	Catch::AssertionHandler catchAssertionHandler("", CATCH_INTERNAL_LINEINFO, AnsiMessage.Get(), Catch::ResultDisposition::Normal);
	catchAssertionHandler.handleMessage(Catch::ResultWas::ExplicitFailure, AnsiMessage.Get());
	catchAssertionHandler.complete();
}

}