// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/OutputDeviceError.h"

namespace UE::LowLevelTests
{

///@brief Used to replace GError so that errors are capture during execution of tests
class FTestRunnerOutputDeviceError : public FOutputDeviceError
{
public:
	explicit FTestRunnerOutputDeviceError(FOutputDeviceError* Error)
		: DeviceError(Error)
	{
	}

	FTestRunnerOutputDeviceError()
		: DeviceError(nullptr)
	{

	}

	virtual void HandleError() override
	{
		if (DeviceError)
		{
			DeviceError->HandleError();
		}
	}

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;

	void SetDeviceError(FOutputDeviceError* Error)
	{
		DeviceError = Error;
	}

	FOutputDeviceError* GetDeviceError() const
	{
		return DeviceError;
	}

private:
	FOutputDeviceError* DeviceError;
};
}