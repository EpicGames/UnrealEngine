// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

namespace UE::LowLevelTests
{

class ITestRunner
{
public:
	/** Returns the test runner if one is active, otherwise null. */
	static ITestRunner* Get();

	/** Returns true if the test runner will perform global setup and teardown. */
	virtual bool HasGlobalSetup() const = 0;
	/** Returns true if the test runner has default logging enabled via UE_LOG. */
	virtual bool HasLogOutput() const = 0;
	/** Returns true if the test runner has extra debug functionality enabled. */
	virtual bool IsDebugMode() const = 0;
	/** Returns a value indicating per test allowed timeout in minutes. */
	virtual int GetTimeoutMinutes() const = 0;

protected:
	ITestRunner();
	virtual ~ITestRunner();
};

} // UE::LowLevelTests
