// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformFile.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "HAL/IPlatformFileModule.h"

class FPlatformFileStub : public IPlatformFile
{
public:
	FPlatformFileStub() : LowerLevel(nullptr) {}
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override { return true; }
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override { LowerLevel = Inner; return true; }
	virtual IPlatformFile* GetLowerLevel() { return LowerLevel; }
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) { LowerLevel = NewLowerLevel; }
	virtual const TCHAR* GetName() const { return TEXT("PlatformFileStub"); }
	virtual bool FileExists(const TCHAR* Filename) override { return false; }
	virtual int64 FileSize(const TCHAR* Filename) override { return -1LL; }
	virtual bool DeleteFile(const TCHAR* Filename) override { return false; }
	virtual bool IsReadOnly(const TCHAR* Filename) override { return false; }
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override { return false; }
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override { return false; }
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override { return FDateTime::MinValue(); }
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override {}
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override { return FDateTime::MinValue(); }
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override { return TEXT(""); }
	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override { return nullptr; }
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override { return nullptr; }
	virtual bool DirectoryExists(const TCHAR* Directory) override { return false; }
	virtual bool CreateDirectory(const TCHAR* Directory) override { return false; }
	virtual bool DeleteDirectory(const TCHAR* Directory) override { return false; }
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override { FFileStatData Data; Data.bIsValid = false; return Data; }
	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override { return true; }
	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override { return true; }
private:
	IPlatformFile* LowerLevel;
};

class FPlatformFileModuleStub : public IPlatformFileModule
{
public:
	virtual IPlatformFile* GetPlatformFile() override
	{
		static TUniquePtr<IPlatformFile> AutoDestroySingleton = MakeUnique<FPlatformFileStub>();
		return AutoDestroySingleton.Get();
	}
};


IMPLEMENT_MODULE(FPlatformFileModuleStub, LowLevelTestsRunner);