// Copyright Epic Games, Inc. All Rights Reserved.

#include <catch2/reporters/catch_reporter_streaming_base.hpp>
#include <catch2/catch_test_case_info.hpp>
#include <catch2/catch_timer.hpp>
#include <catch2/reporters/catch_reporter_registrars.hpp>
#include <catch2/internal/catch_xmlwriter.hpp>
#include "Misc/StringBuilder.h"

namespace UE::LowLevelTests
{
/** Simple reporter for the low level tests

<testrun>
	<testcase name="test::name">
		<failure>
			all error text 
		</failure>
		<result success="true/false" duration="double seconds"/>
	</testcase>
</testrun>

no tags, no sections. errors are aggregated together.
**/
class FUnrealReporter : public Catch::StreamingReporterBase
{
public:
	using Catch::StreamingReporterBase::StreamingReporterBase;

	FUnrealReporter(Catch::ReporterConfig&& Config)
		: Catch::StreamingReporterBase(CATCH_MOVE(Config))
		, Xml(m_stream)
	{
		m_preferences.shouldRedirectStdOut = true;
		m_preferences.shouldReportAllAssertions = true;
	}

	void testRunStarting(Catch::TestRunInfo const& TestInfo)
	{
		StreamingReporterBase::testRunStarting(TestInfo);
		Xml.startElement("testrun");
	}

	virtual void testRunEnded(Catch::TestRunStats const&) override
	{
		Xml.endElement();
	}

	void testCaseStarting(Catch::TestCaseInfo const& TestCaseInfo) override
	{
		Timer.start();
		Xml.startElement("testcase");
		Xml.writeAttribute("name", TestCaseInfo.name);
	}

	virtual void testCaseEnded(Catch::TestCaseStats const& TestCaseStats) override
	{
		using namespace Catch;
		double Duration = Timer.getElapsedSeconds();

		if (bHasFailedOnce)
		{
			Xml.endElement(); //end failure element
		}
		bHasFailedOnce = false;
		{
			//using an element instead of an attribute on "testcase" to make streaming to the xml writer easier
			//otherwise any failures would have to be stored as a member
			XmlWriter::ScopedElement ResultElement = Xml.scopedElement("result");
			ResultElement.writeAttribute("success", TestCaseStats.totals.assertions.allOk());
			ResultElement.writeAttribute("duration", Duration);
		}
		Xml.endElement();
	}

	virtual void skipTest(Catch::TestCaseInfo const& TestInfo) override
	{
		Catch::XmlWriter::ScopedElement TestCaseElement = Xml.scopedElement("testcase");
		TestCaseElement.writeAttribute("name", TestInfo.name);
		TestCaseElement.writeAttribute("skipped", "true");
	}

	static std::string getDescription()
	{
		return "Reporter for LowLevelTests";
	}

	void assertionEnded(Catch::AssertionStats const& Stats) override
	{
		using namespace Catch;

		AssertionResult const& Result = Stats.assertionResult;

		if (!Result.isOk())
		{
			const char* ElementName;
			switch (Result.getResultType())
			{
			case ResultWas::ThrewException:
			case ResultWas::FatalErrorCondition:
			case ResultWas::ExplicitFailure:
			case ResultWas::ExpressionFailed:
			case ResultWas::DidntThrowException:
				ElementName = "failure";
				break;

			default:
				ElementName = "internalError";
				break;
			}

			if (!bHasFailedOnce)
			{
				bHasFailedOnce = true;
				Xml.startElement(ElementName);
			}
			TAnsiStringBuilder<1024> Stream;
			if (Stats.totals.assertions.total() > 0)
			{
				Stream << "FAILED" << ":\n";
				if (Result.hasExpression())
				{
					Stream << "  " 
						<< Result.getExpressionInMacro().c_str()
						<< "\n";
				}
				if (Result.hasExpandedExpression())
				{
					Stream << "with expansion:\n" 
						<< Result.getExpandedExpression().c_str() << "\n";
				}
			}
			else 
			{
				Stream << '\n';
			}

			if (!Result.getMessage().empty())
			{
				Stream << Result.getMessage().data() << '\n';
			}
			for (auto const& Msg : Stats.infoMessages)
			{
				if (Msg.type == ResultWas::Info)
				{
					Stream << Msg.message.data() << '\n';
				}
			}

			Stream << "at " << Result.getSourceInfo().file << '(' << (uint64)Result.getSourceInfo().line << ')';
			Xml.writeText(*Stream, XmlFormatting::Newline);
		}
	}


private:
	Catch::XmlWriter Xml;
	Catch::Timer Timer;
	bool bHasFailedOnce = false;
};


CATCH_REGISTER_REPORTER("unreal", FUnrealReporter)

}