// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//Read command line arguments from file to add on top of the real ones. Specify valid directory with separator at the end and path real command line arguments. 

#ifdef __cplusplus
extern "C"
{
#endif

const char** ReadAndAppendAdditionalArgs(const char* FileDir, int* OutNumArgs, const char** Argv, int Argc);

#ifdef __cplusplus
} //extern "C"
#endif