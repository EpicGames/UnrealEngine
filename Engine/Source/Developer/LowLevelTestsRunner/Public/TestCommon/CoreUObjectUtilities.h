// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_COREUOBJECT

void InitCoreUObject();

void CleanupCoreUObject();

#endif // WITH_COREUOBJECT