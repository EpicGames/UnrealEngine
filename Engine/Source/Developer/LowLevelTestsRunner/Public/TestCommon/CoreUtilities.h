// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/TaskGraphInterfaces.h"

void InitGWarn();

void InitIOThreadPool(bool MultiThreaded = false, int32 StackSize = 128 * 1024);

void InitThreadPool(bool MultiThreaded = false, int32 StackSize = 128 * 1024);

void InitBackgroundPriorityThreadPool(bool MultiThreaded = false, int32 StackSize = 128 * 1024);

void InitAllThreadPools(bool MultiThreaded = false);

void CleanupThreadPool();

void CleanupIOThreadPool();

void CleanupBackgroundPriorityThreadPool();

void CleanupAllThreadPools();

void InitTaskGraph(bool MultiThreaded = false, ENamedThreads::Type ThreadToAttach = ENamedThreads::GameThread);

void CleanupTaskGraph();

void InitTaskGraphAndDependencies(bool MultiThreaded = false);

void CleanupTaskGraphAndDependencies();

void CleanupPlatform();