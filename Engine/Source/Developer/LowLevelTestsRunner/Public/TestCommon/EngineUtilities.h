// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ENGINE

void InitAsyncQueues();
void InitRendering();
void InitEngine();
void CleanupEngine();

#endif // WITH_ENGINE
