// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/CString.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Containers/UnrealString.h"
#include "Containers/StringFwd.h"

#include "../TestHarness.h"

namespace
{
	FString GetStringValueToDisplay(FStringView Value)
	{
		if (Value.GetData())
		{
			return FString::Printf(TEXT("\"%.*s\""), Value.Len(), Value.GetData());
		}
		else
		{
			return TEXT("nullptr");
		}
	}

	FString GetStringValueToDisplay(FUtf8StringView Value)
	{
		if (Value.GetData())
		{
			return FString::Printf(TEXT("\"%s\""), *WriteToString<128>(Value));
		}
		else
		{
			return TEXT("nullptr");
		}
	}

	bool TestTrue(const TCHAR* What, bool Value)
	{
		if (!Value)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be true."), What));
			return false;
		}
		return true;
	}

	bool TestTrue(const FString& What, bool Value)
	{
		return TestTrue(*What, Value);
	}

	bool TestFalse(const TCHAR* What, bool Value)
	{
		if (Value)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be false."), What));
			return false;
		}
		return true;
	}

	bool TestFalse(const FString& What, bool Value)
	{
		return TestFalse(*What, Value);
	}

	bool TestEqual(const TCHAR* What, const int32 Actual, const int32 Expected)
	{
		if (Actual != Expected)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %d, but it was %d."), What, Expected, Actual));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const int64 Actual, const int64 Expected)
	{
		if (Actual != Expected)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %" "lld" ", but it was %" "lld" "."), What, Expected, Actual));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const SIZE_T Actual, const SIZE_T Expected)
	{
		if (Actual != Expected)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %zu, but it was %zu."), What, Expected, Actual));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		if (!FMath::IsNearlyEqual(Actual, Expected, Tolerance))
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %f, but it was %f within tolerance %f."), What, Expected, Actual, Tolerance));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		if (!FMath::IsNearlyEqual(Actual, Expected, Tolerance))
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %f, but it was %f within tolerance %f."), What, Expected, Actual, Tolerance));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const FVector Actual, const FVector Expected, float Tolerance)
	{
		if (!Expected.Equals(Actual, Tolerance))
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s within tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const FTransform Actual, const FTransform Expected, float Tolerance)
	{
		if (!Expected.Equals(Actual, Tolerance))
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s within tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const FRotator Actual, const FRotator Expected, float Tolerance)
	{
		if (!Expected.Equals(Actual, Tolerance))
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s within tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const FColor Actual, const FColor Expected)
	{
		if (Expected != Actual)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const FLinearColor Actual, const FLinearColor Expected)
	{
		if (Expected != Actual)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *Expected.ToString(), *Actual.ToString()));
			return false;
		}
		return true;
	}

	bool TestEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
	{
		bool bAreEqual = (Actual && Expected) ? (FCString::Stricmp(Actual, Expected) == 0) : (Actual == Expected);

		if (!bAreEqual)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
		}

		return bAreEqual;
	}

	bool TestEqual(const TCHAR* What, FStringView Actual, FStringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::IgnoreCase) != 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::IgnoreCase) != 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
	{
		bool bAreEqual = (Actual && Expected) ? (FCString::Strcmp(Actual, Expected) == 0) : (Actual == Expected);

		if (!bAreEqual)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
		}

		return bAreEqual;
	}

	bool TestEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::CaseSensitive) != 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::CaseSensitive) != 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	UE_DEPRECATED(5.5, "Use TestEqual instead (string tests are case insensitive by default)")
	bool TestEqualInsensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestEqual(What, Actual, Expected);
	}

	bool TestEqual(const FString& What, const int32 Actual, const int32 Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, const float Actual, const float Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const double Actual, const double Expected, double Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FVector Actual, const FVector Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FTransform Actual, const FTransform Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FRotator Actual, const FRotator Expected, float Tolerance = UE_KINDA_SMALL_NUMBER)
	{
		return TestEqual(*What, Actual, Expected, Tolerance);
	}

	bool TestEqual(const FString& What, const FColor Actual, const FColor Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestEqual(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	template<typename ValueType>
	bool TestEqual(const TCHAR* What, const ValueType& Actual, const ValueType& Expected)
	{
		if (Actual != Expected)
		{
			FAIL_CHECK(FString::Printf(TEXT("%s: The two values are not equal."), What));
			return false;
		}
		return true;
	}

	template<typename ValueType>
	bool TestEqual(const FString& What, const ValueType& Actual, const ValueType& Expected)
	{
		return TestEqual(*What, Actual, Expected);
	}

	bool TestNotEqual(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
	{
		bool bAreDifferent = (Actual && Expected) ? (FCString::Stricmp(Actual, Expected) != 0) : (Actual != Expected);

		if (!bAreDifferent)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
		}

		return bAreDifferent;
	}

	bool TestNotEqual(const TCHAR* What, FStringView Actual, FStringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::IgnoreCase) == 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestNotEqual(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::IgnoreCase) == 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestNotEqualSensitive(const TCHAR* What, const TCHAR* Actual, const TCHAR* Expected)
	{
		bool bAreDifferent = (Actual && Expected) ? (FCString::Strcmp(Actual, Expected) != 0) : (Actual != Expected);

		if (!bAreDifferent)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
		}

		return bAreDifferent;
	}

	bool TestNotEqualSensitive(const TCHAR* What, FStringView Actual, FStringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::CaseSensitive) == 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	bool TestNotEqualSensitive(const TCHAR* What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		if (Actual.Compare(Expected, ESearchCase::CaseSensitive) == 0)
		{
			FAIL_CHECK(FString::Printf(TEXT("Expected '%s' to differ from %s, but it was %s."), What, *GetStringValueToDisplay(Expected), *GetStringValueToDisplay(Actual)));
			return false;
		}

		return true;
	}

	template<typename ValueType> bool TestNotEqual(const TCHAR* Description, const ValueType& Actual, const ValueType& Expected)
	{
		if (Actual == Expected)
		{
			FAIL_CHECK(FString::Printf(TEXT("%s: The two values are equal."), Description));
			return false;
		}
		return true;
	}

	bool TestNotEqual(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestNotEqual(*What, Actual, Expected);
	}

	bool TestNotEqual(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestNotEqual(*What, Actual, Expected);
	}

	bool TestNotEqual(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestNotEqual(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, const TCHAR* Actual, const TCHAR* Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, FStringView Actual, FStringView Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	bool TestNotEqualSensitive(const FString& What, FUtf8StringView Actual, FUtf8StringView Expected)
	{
		return TestNotEqualSensitive(*What, Actual, Expected);
	}

	template<typename ValueType> bool TestNotEqual(const FString& Description, const ValueType& Actual, const ValueType& Expected)
	{
		return TestNotEqual(*Description, Actual, Expected);
	}

	#define CHECK_EQUALS(What, X, Y)				TestEqual(What, X, Y);
	#define CHECK_EQUALS_SENSITIVE(What, X, Y)		TestEqualSensitive(What, X, Y);
	#define CHECK_NOT_EQUALS(What, X, Y)			TestNotEqual(What, X, Y);
	#define CHECK_NOT_EQUALS_SENSITIVE(What, X, Y)	TestNotEqualSensitive(What, X, Y);

}
