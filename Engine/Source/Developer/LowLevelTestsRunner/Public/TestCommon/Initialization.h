// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

void SaveDefaultPlatformFile();
void UsePlatformFileStubIfRequired();
void UseDefaultPlatformFile();

void SetProjectNameAndDirectory();

void InitAllThreadPoolsEditorEx(bool MultiThreaded);

void InitOutputDevicesEx();

void InitStats();

void InitAll(bool bAllowLogging, bool bMultithreaded);

void CleanupLocalization();
void CleanupAll();