// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef UE_ENABLE_TESTHARNESS_ENGINE_SUPPORT
#define UE_ENABLE_TESTHARNESS_ENGINE_SUPPORT 1
#endif

#if UE_ENABLE_TESTHARNESS_ENGINE_SUPPORT
#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Templates/TupleFwd.h"

#if PLATFORM_MAC
// Fix type redefinition error of FVector
#include "HAL/PlatformMath.h"
#endif // PLATFORM_MAC

#include <ostream>

std::ostream& operator<<(std::ostream& Stream, const TCHAR* Value);
std::ostream& operator<<(std::ostream& Stream, const FString& Value);
std::ostream& operator<<(std::ostream& Stream, const FAnsiStringView& Value);
std::ostream& operator<<(std::ostream& Stream, const FWideStringView& Value);
std::ostream& operator<<(std::ostream& Stream, const FUtf8StringView& Value);
std::ostream& operator<<(std::ostream& Stream, const FAnsiStringBuilderBase& Value);
std::ostream& operator<<(std::ostream& Stream, const FWideStringBuilderBase& Value);
std::ostream& operator<<(std::ostream& Stream, const FUtf8StringBuilderBase& Value);

enum class ESPMode : uint8;
template <class ObjectType, ESPMode InMode> class TSharedRef;
template <class ObjectType, ESPMode InMode> class TSharedPtr;

template <typename ObjectType, ESPMode Mode>
std::ostream& operator<<(std::ostream& Stream, const TSharedRef<ObjectType, Mode>& Value)
{
	return Stream << &Value.Get();
}

template <typename ObjectType, ESPMode Mode>
std::ostream& operator<<(std::ostream& Stream, const TSharedPtr<ObjectType, Mode>& Value)
{
	return Stream << Value.Get();
}

template <typename KeyT, typename ValueT>
inline bool operator==(const TMap<KeyT, ValueT>& Left, const TMap<KeyT, ValueT>& Right)
{
	bool bIsEqual = Left.Num() == Right.Num();
	if (bIsEqual)
	{
		for (const auto& Pair : Left)
		{
			const ValueT* RightValue = Right.Find(Pair.Key);
			bIsEqual = bIsEqual && RightValue != nullptr && Pair.Value == *RightValue;
		}
	}
	return bIsEqual;
}

#if defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)
THIRD_PARTY_INCLUDES_START
#endif // defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)

#ifdef _MSC_VER
#pragma pack(push, 8)
#pragma warning(push)
#pragma warning(disable: 4005) // 'identifier': macro redefinition
#pragma warning(disable: 4582) // 'type': constructor is not implicitly called
#pragma warning(disable: 4583) // 'type': destructor is not implicitly called
#endif
#include <catch2/catch_test_macros.hpp>
#include <catch2/catch_version_macros.hpp>

#if CATCH_VERSION_MAJOR >= 3 && CATCH_VERSION_MINOR >= 11
#include <catch2/internal/catch_unreachable.hpp>
#endif

#ifdef _MSC_VER
#pragma warning(pop)
#pragma pack(pop)
#endif

#if defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)
THIRD_PARTY_INCLUDES_END
#endif // defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)

#define LLT_CONCAT_(x, y) x##y
#define LLT_CONCAT(x, y) LLT_CONCAT_(x,y)

#define DISABLED_TEST_CASE_METHOD_INTERNAL(TestName, ClassName, ...) \
	CATCH_INTERNAL_START_WARNINGS_SUPPRESSION \
	CATCH_INTERNAL_SUPPRESS_GLOBALS_WARNINGS \
	namespace \
	{ \
		struct TestName : INTERNAL_CATCH_REMOVE_PARENS(ClassName) \
		{ \
			void test(); \
		}; \
	} \
	CATCH_INTERNAL_STOP_WARNINGS_SUPPRESSION \
	void TestName::test()

#define DISABLED_TEST_CASE(...) static void LLT_CONCAT(disabled_test_,__LINE__)()
#define DISABLED_TEST_CASE_METHOD(ClassName, ...) \
	DISABLED_TEST_CASE_METHOD_INTERNAL(INTERNAL_CATCH_UNIQUE_NAME( CATCH2_INTERNAL_TEST_ ), ClassName, __VA_ARGS__)
#define DISABLED_SCENARIO(...) static void LLT_CONCAT(disabled_scenario_,__LINE__)()
#define DISABLED_SECTION(...) auto LLT_CONCAT(disabled_section_,__LINE__) = []()

// Tell Catch how to print TTuple<KeyType, ValueType>
template <typename KeyType, typename ValueType>
struct Catch::StringMaker<TTuple<KeyType, ValueType>>
{
	static std::string convert(const TTuple<KeyType, ValueType>& Value)
	{
		return "{ " + StringMaker<KeyType>::convert(Value.Key) + " , " + StringMaker<ValueType>::convert(Value.Value) + " }";
	}
};

#define TEST_CASE_NAMED(ClassName, ...) TEST_CASE(__VA_ARGS__)

#define VERIFY(What, Actual)\
	CAPTURE(What);\
	CHECK(Actual == true)

#define ADD_WARNING(What)\
	WARN(What)

#define ADD_ERROR(What)\
	FAIL(What)

#define FAIL_ON_MESSAGE(What)

#define CHECK_EQUAL(Actual, Expected)\
	CHECK(Actual == Expected)

#define CHECK_NOT_EQUAL(Actual, Expected)\
	CHECK(Actual != Expected)

#define CHECK_MESSAGE(What, Value) do { \
	INFO(What); \
	CHECK((Value)==true); \
} while (false)

#define CHECK_FALSE_MESSAGE(What, Value) do { \
	INFO(What); \
	CHECK_FALSE(Value); \
} while (false)

#define REQUIRE_EQUAL(Actual, Expected)\
	REQUIRE(Actual == Expected)

#define REQUIRE_NOT_EQUAL(Actual, Expected)\
	REQUIRE(Actual != Expected)

#define REQUIRE_MESSAGE(What, Value) do { \
	INFO(What); \
	REQUIRE((Value)==true); \
} while (false)

#define TEST_TRUE(What, Value)\
	INFO(What);\
	CHECK((Value)==true)

#define TEST_FALSE(What, Value)\
	INFO(What);\
	CHECK((Value)==false)

#define TEST_IF_EQUAL(What, Actual, Expected)\
	INFO(What);\
	CHECKED_IF(Actual == Expected)

#define TEST_EQUAL(What, Actual, Expected)\
	INFO(What);\
	CHECK((Actual) == (Expected))

#define TEST_EQUAL_STR(What, Expected, Actual)\
	INFO(What);\
	CAPTURE(Actual);\
	CHECK(FCString::Strcmp(ToCStr((Expected)), ToCStr((Actual))) == 0)

#define TEST_NOT_EQUAL(What, Actual, Expected)\
	INFO(What);\
	CHECK((Actual) != (Expected))

#define TEST_NULL(What, Value)\
	INFO(What);\
	CHECK((Value)==nullptr)

#define TEST_NOT_NULL(What, Value)\
	INFO(What);\
	CHECK((Value)!=nullptr)

#define TEST_VALID(What, Value)\
	INFO(What);\
	CHECK(Value.IsValid()==true)

#define TEST_INVALID(What, Value)\
	INFO(What);\
	CHECK(Value.IsValid()==false)

#endif // UE_ENABLE_TESTHARNESS_ENGINE_SUPPORT