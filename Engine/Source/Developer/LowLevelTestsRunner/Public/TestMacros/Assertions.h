// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/AssertionMacros.h"
#include "Tests/EnsureScope.h"
#include "Tests/CheckScope.h"

//requires that an UE `ensure` fails in this call
#define REQUIRE_ENSURE(...) INTERNAL_UE_ENSURE( "REQUIRE_ENSURE", DO_ENSURE, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

//requires that an UE `ensure` fails with a message that matches the supplied message
#define REQUIRE_ENSURE_MSG(msg, ...) INTERNAL_UE_ENSURE_MSG(msg, "REQUIRE_ENSURE", DO_ENSURE, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

//checks that an UE `ensure` fails in this call
#define CHECK_ENSURE(...) INTERNAL_UE_ENSURE( "CHECK_ENSURE", DO_ENSURE, Catch::ResultDisposition::ContinueOnFailure, #__VA_ARGS__, __VA_ARGS__ )

//checks that an UE `ensure` fails with a message that matches the supplied message
#define CHECK_ENSURE_MSG(msg, ...) INTERNAL_UE_ENSURE_MSG(msg, "CHECK_ENSURE", DO_ENSURE, Catch::ResultDisposition::ContinueOnFailure, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `check` fails in this call
#define REQUIRE_CHECK(...) INTERNAL_UE_CHECK( "REQUIRE_CHECK", DO_CHECK, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `check` fails in this call contains the supplies message
#define REQUIRE_CHECK_MSG(msg, ...) INTERNAL_UE_CHECK_MSG(msg, "REQUIRE_CHECK", DO_CHECK, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `checkSlow` fails in this call
#define REQUIRE_CHECK_SLOW(...) INTERNAL_UE_CHECK( "REQUIRE_CHECK_SLOW", DO_GUARD_SLOW, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

//requires that a UE `checkSlow` fails in this call contains the supplies message
#define REQUIRE_CHECK_SLOW_MSG(msg, ...) INTERNAL_UE_CHECK_MSG(msg, "REQUIRE_CHECK_SLOW", DO_GUARD_SLOW, Catch::ResultDisposition::Normal, #__VA_ARGS__, __VA_ARGS__ )

#define INTERNAL_UE_ENSURE( macroName, doEnsure, resultDisposition, ensureExpr, ... ) \
	do { \
		Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, ensureExpr, resultDisposition ); \
		INTERNAL_CATCH_TRY { \
			FEnsureScope scope; \
			static_cast<void>(__VA_ARGS__); \
			bool bEncounteredEnsure = scope.GetCount() > 0; \
			if (doEnsure && !bEncounteredEnsure) \
				catchAssertionHandler.handleMessage( Catch::ResultWas::ExplicitFailure ,"Expected failure of `ensure` not received."); \
		} INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
		catchAssertionHandler.complete(); \
	} while(false) \

#define INTERNAL_UE_ENSURE_MSG(msg, macroName, doEnsure, resultDisposition, ensureExpr, ... ) \
	do { \
		Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, ensureExpr, resultDisposition ); \
		INTERNAL_CATCH_TRY { \
			FEnsureScope scope(msg); \
			static_cast<void>(__VA_ARGS__); \
			bool bEncounteredEnsure = scope.GetCount() > 0; \
			if (doEnsure && !bEncounteredEnsure) \
				catchAssertionHandler.handleMessage( Catch::ResultWas::ExplicitFailure, ( Catch::MessageStream() << "Expected failure of `ensure` with message '" << msg << "' not received" + ::Catch::StreamEndStop() ).m_stream.str() ); \
		} INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
		catchAssertionHandler.complete(); \
	} while(false) \

#define INTERNAL_UE_CHECK(macroName, doCheck, resultDisposition, checkExpr, ... ) \
	do { \
		Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, checkExpr, resultDisposition ); \
		INTERNAL_CATCH_TRY { \
			FCheckScope scope; \
			(void)(__VA_ARGS__); \
			bool bEncounteredEnsure = scope.GetCount() > 0; \
			if (doCheck && !bEncounteredEnsure) \
				catchAssertionHandler.handleMessage( Catch::ResultWas::ExplicitFailure, "Expected failure of `check` not received"); \
		} INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
		catchAssertionHandler.complete(); \
	} while(false) \

#define INTERNAL_UE_CHECK_MSG(msg, macroName, doCheck, resultDisposition, checkExpr, ... ) \
	do { \
		Catch::AssertionHandler catchAssertionHandler( macroName##_catch_sr, CATCH_INTERNAL_LINEINFO, checkExpr, resultDisposition ); \
		INTERNAL_CATCH_TRY { \
			FCheckScope scope(msg); \
			(void)(__VA_ARGS__); \
			bool bEncounteredEnsure = scope.GetCount() > 0; \
			if (doCheck && !bEncounteredEnsure) \
				catchAssertionHandler.handleMessage( Catch::ResultWas::ExplicitFailure, ( Catch::MessageStream() << "Expected failure of `check` containing message '" << msg << "' not received" + ::Catch::StreamEndStop() ).m_stream.str() ); \
		} INTERNAL_CATCH_CATCH( catchAssertionHandler ) \
		catchAssertionHandler.complete(); \
	} while(false) \

