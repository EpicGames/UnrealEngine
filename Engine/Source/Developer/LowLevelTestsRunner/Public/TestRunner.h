// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include COMPILED_PLATFORM_HEADER_WITH_PREFIX(Platform,TestRunner.h)

#include "CommandLineUtil.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Logging/LogSuppressionInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"
#include "Templates/SharedPointer.h"
#include <cstring>
#include <thread>
#include <chrono>

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 4005)
#endif

#if __cplusplus
	extern "C" const char* GetProcessExecutablePath();
#else
	const char* GetProcessExecutablePath();
#endif
const char* GetCacheDirectory();

DECLARE_DELEGATE(FGlobalSetupHandler);
DECLARE_DELEGATE(FGlobalPlatformSetupHandler);

class FTestDelegates
{
public:
	static FGlobalSetupHandler& GetGlobalSetup()
	{
		static TUniquePtr<FGlobalSetupHandler> GlobalSetup = MakeUnique<FGlobalSetupHandler>();
		return *GlobalSetup.Get();
	}

	static FGlobalSetupHandler& GetGlobalTeardown()
	{
		static TUniquePtr<FGlobalSetupHandler> GlobalTeardown = MakeUnique<FGlobalSetupHandler>();
		return *GlobalTeardown.Get();
	}

	static FGlobalPlatformSetupHandler& GetGlobalPlatformSetup()
	{
		static TUniquePtr<FGlobalPlatformSetupHandler> GlobalPlatformSetup = MakeUnique<FGlobalPlatformSetupHandler>();
		return *GlobalPlatformSetup.Get();
	}
};

int RunTests(int argc, const char* argv[]);