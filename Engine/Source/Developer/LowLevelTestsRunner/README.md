
# Low Level Tests

**Low Level Tests (LLTs)** is a testing framework for lightweight, module-centric tests in Unreal Engine (UE). Low-Level Tests are written with the Catch2 test framework. You can build and execute Low-Level Tests for all platforms supported by UE. Low-Level Tests include the following test types:
* **Explicit**: Self-contained tests defined by a module and target pair.
* **Implicit**: Tests that live inside the module being tested and do not require a module and target pair.

Both explicit and implicit tests can use any of the following test methodologies:
* **Unit**: Test a standalone section or unit of code.
* **Integration**: Test multiple sections or units of code combined as a group.
* **Functional**: Test specific functionality of a feature or use case.
* **Smoke**: Quick validation of a feature or use case.
* **End-to-end**: Test several stages of a feature.
* **Performance**: Benchmark the running time of a feature.
* **Stress**: Try to break functionality by straining the system.

You can write Low-Level Tests with either of the following test paradigms:
* **Test-Driven Development (TDD)**
* **Behavior-Driven Development (BDD)**

## Why Use Low-Level Tests

What sets Low-Level Tests apart from other UE test frameworks is their minimality with respect to compilation and runtime resources. LLTs are made to work with various UE features, including: [UObjects](https://docs.unrealengine.com/5.3/en-US/objects-in-unreal-engine), [assets](https://docs.unrealengine.com/5.3/en-US/working-with-assets-in-unreal-engine), [engine components](https://docs.unrealengine.com/5.3/en-US/basic-components-in-unreal-engine), and more. Low-Level Tests in Unreal Engine are written with Catch2 — a modern C++ test framework — extended to include test grouping and lifecycle events, as well as other features that work well with the modular architecture of Unreal Engine.

## Guide to Low-Level Tests Documentation
This documentation covers types of Low-Level Tests, how to write and build them. Common to all use cases is the Build and Run documentation section, which is essential for learning how to run your tests.

### Unreal Engine Low-Level Tests Documentation

The areas covered in the Unreal Engine Low-Level Test Documentation include:
* [Types of Low-Level Tests](#types-of-low-level-tests)
* [Write Low-Level Tests](#write-low-level-tests)
* [Build and Run Low-Level Tests](#build-and-run-low-level-tests)

### Catch2 Documentation

This documentation does not provide a comprehensive resource for the features that Catch2 provides. For more information about Catch2, see the [Catch2 repository](https://github.com/catchorg/Catch2) on GitHub. For detailed information about Catch2, including writing tests in Catch2, see the [Catch2 Documentation](https://github.com/catchorg/Catch2/tree/devel/docs).

## Testing Methodologies
This section gives you a brief overview of the different testing methods that Low-Level Tests can help you implement.

### Unit tests

**Unit tests** test one unit of code, typically a single method within a class. They rely on mocking inputs and external dependencies such as servers or databases. Typically, you write one unit test suite per tested class. The goal of a unit test is to cover the public interface of a class, as well as the different code paths within a method.

Most tests are not unit tests because they are very restrictive in how they should be written, and the target code might not be testable in this strict way. Unit tests should only require minimal special global setup (set up mocks, stubs, and fakes) and teardown, and they should be able to be run independently between test suites. Unit tests should not have an order dependency on other unit tests. Unit tests are very fast — they take seconds or less to run.

### Integration tests

**Integration tests** test multiple units of code together, typically two or more classes or methods. They can require global setup, such as loading modules or an external resource. They are less restrictive than unit tests and are more common, but it can be harder to cover branches in code (if conditions etc.). Integration tests are usually slower than unit tests — they take up to seconds to run.

### Functional tests

**Functional tests** test a specific functionality, typically a single feature or a use case. They often require global setup and teardown to manage external resources. These are the most common types of tests, and they can vary greatly in complexity. Functional tests can take seconds to minutes and rarely hours to run. They are usually slower than integration tests.

### Smoke tests

**Smoke tests** provide quick validation of a feature or use case. These tests cover minimum acceptance criteria. They can be run at the startup of an application if it takes a couple of seconds or in development builds. Typically included in continuous integration, iterative builds.

### End to end tests

**End-to-end tests** run through several stages of a feature as opposed to just a segment. They are heavier tests that might require minutes or more to complete. End-to-end tests usually have checkpoints with preconditions that can stop the test.

### Performance and stress tests

**Performance tests** are often benchmarks and are typically long running. Stress tests target one functionality, and they try to break it through repeated actions or by putting the system under strain. Both types typically capture performance metrics that are compared with baselines. Both types can be slow and usually take up to hours to complete but there's no general rule as to how much time they should run for. Some performance tests might only require seconds or a few minutes to complete.

## Types of Low-Level Tests

The Low-Level Tests (LLTs) framework recognizes two different types of tests:
* **Explicit**: Self-contained tests defined by a module build-target pair.
* **Implicit**: Tests that live inside the tested module and do not require a build-target pair.

## Explicit tests

**Explicit Tests** are self-contained tests defined by a module build-target pair. Explicit tests are designed to be lightweight in terms of compilation time and run time. They are called explicit because they require explicit UE module build and target files, as opposed to implicit tests, which do not require them. This means explicit tests require both a `.Build.cs` and a `.Target.cs` file while implicit tests do not. For more information about the distinction between explicit and implicit tests, see the [Implicit Tests](#implicit-tests) section.

### Create an Explicit Test

Follow these steps to create your explicit test:

1. In the *Source\Programs* directory, create a new directory with the same name as the module you want to test and add the .Build.cs file in this directory.
> **_TIP:_**  To see an example, the directory `Engine/Source/Programs/LowLevelTests` contains an explicit test target named Foundation Tests.

2. Inherit your module class from `TestModuleRules`.
    * If you are writing a test for a plugin, place the new module inside a `Tests` directory at the same level as the plugin's Source directory.
    * If you are building a test module that does not use `Catch2`, inherit the base constructor with the second parameter set to false: `base(Target, false)`.
3. Implement a static constructor for the class and initialize `TestMetadata` explicitly.
    * The dat stored in this object is used to generate BuildGraph script test metadata.
    * For more information about BuildGraph script generation, see the [Generate BuildGraph Script Metadata](#generate-buildgraph-script-metadata-files) section.
4. Suppose that you have an explicit test module titled `UEModuleTests`. Your explicit test `.Build.cs` file should look similar to this:
	**UEModuleTests\UEModuleTests.Build.cs**
	<font size="2">
	```csharp
	public class UEModuleTests : TestModuleRules
	{
        static UEModuleTests()
        {
          if (InTestMode)
          {
            TestMetadata = new Metadata();
		    TestMetadata.TestName = "UEModule";
		    TestMetadata.TestShortName = "UE Module";
		    // any other additional properties
          }
        }

        public UEModuleTests(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateIncludePaths.AddRange(
                // Any private include paths
            );
			PrivateDependencyModuleNames.AddRange(
				// Any private dependencies to link against
			);

			// Other types of dependencies or module specific logic
		}
	}
	```
	</font>

	You can also extend this module class for a NDA platform, in which case you need to provide a static constructor in that class and call `TestMetadata.SupportedPlatforms.Add(...)` with the platform as parameter - do not re-initialize TestMetadata so that all the properties set in the base class are kept.
5. Add a test target file (`.Target.cs`) with a class that inherits from `TestTargetRules`.
6. Override the default compilation flags if necessary.
    * Aim for a minimal, testable module free of default features that don't add value to low-level testing.
	* You can specify the supported platforms individually. The default platforms are: Win64, Mac, Linux, and Android.
	* You can enable project-specific global definitions and set Catch2 definitions, such as those needed for benchmarking support.
7. Your explicit tests .Target.cs file should look similar to this:
	**UEModuleTests\UEModuleTests.Target.cs**
	<font size="2">
	```csharp
	[SupportedPlatforms(UnrealPlatformClass.All)]
	public class UEModuleTestsTarget : TestTargetRules
	{
		public UEModuleTestsTarget(TargetInfo Target) : base(Target)
		{
			// Setup like any other target: set compilation flags, global definitions etc.
			GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");
		}
	}
	```
	</font>

#### Next Steps

Now you can write C++ test files in the `Private` folder of the module and write Catch2 tests in these files. For testing tips and best practices, see the [Write Low-Level Tests](#write-low-level-tests) section. Lastly, learn how to build and run your tests. There is more than one way to build and execute low-level tests. See the [Build and Run Low-Level Tests](#build-and-run-low-level-tests) section to select the best method for your development needs.

### Generate BuildGraph Script Metadata Files

If you want to build and run your tests with BuildGraph, you need generate BuildGraph script metadata files for explicit tests.
All classes inheriting TestModuleRules have a static property called TestMetadata whose properties are translated into BuildGraph metadata and instructions.

Run the following command at the root of the engine:
```
.\RunUBT.bat -Mode=Test -GenerateMetadata
```

If your tests live inside a project, use `-ProjectFile=` with the absolute path to the *.uproject* file to generate metadata in the project's directory under the `Build\LowLevelTests` folder.


This generates test metadata `.xml` files in the `Engine/Build/LowLevelTests/<TEST_NAME>.xml` folder for each test target, where `<TEST_NAME>` is the name of your test target. For NDA platforms, these files are generated under `Platforms/<PLATFORM_NAME>/Build/LowLevelTests/<TEST_NAME>.xml`. An additional `General.xml` file is optionally present next to the test files containing global properties.

These files are overwritten every time the command is executed and their contents depends on the TestModuleRules inherited class' `TestMetadata` object. The folders and files that are accessed by project file generation must be writable. Typically, these files are read-only when under source control, so check them out or make them writeable before generation.

> **_TIP_**: To see an example, the directory Engine/Build/LowLevelTests contains an .xml file named Foundation. This is the generated BuildGraph metadata for the Foundation Tests.

### Explicit Tests Reference

#### Test Module Rules Reference

The `TestModuleRules` class extends `ModuleRules` and adds a static `TestMetadata` filed which is used to generate BuildGraph test metadata `.xml` files. With a `Metadata` object type, you can set the following properties:
* `TestName`: The name of your tests used by the BuildGraph script to generate test-specific properties. This field cannot contain spaces.
* `TestShortName`: The short name of your tests used for display in the build system. This field can contain spaces.
* `ReportType`: The Catch2 report type. The most common report types are console and xml. For more information about Catch2 report types, see the external Catch2 documentation.
* `Disabled`: Whether the test is disabled. If true, this test is excluded from the BuildGraph graph.
* `InitialExtraArgs`: Command-line arguments that are prepended in front of other arguments for the `RunLowLevelTests` Gauntlet command. These are typically Gauntlet feature-enabling arguments that only apply to some tests. For example, `-printreport`, which prints the report to `stdout` at the end of test execution.
* `HasAfterSteps`: If true, tests must provide a BuildGraph `Macro` with the name `<TEST_NAME>AfterSteps` that include any cleanup or steps necessary to run after the test execution. For example, this could be running a database script that cleans up any leftover test data.
* `UsesCatch2`: This property allows you to choose your test framework. Some tests don't use Catch2; they might use `GoogleTest` for example. If you choose your own test framework, ensure that you implement support for reporting and other features in the `RunLowLevelTests` Gauntlet command.
* `PlatformTags`: Platform-specific list of tags. For example, use this to exclude unsupported tests on a given platform.
* `PlatformCompilationExtraArgs`: Any extra compilation arguments that a platform might require.
* `PlatformsRunUnsupported`: Add an exception and can serve as a compilation safety net in the BuildGraph script until running support is implemented. For example, if a platform only supports compilation but lacks low-level test running capabilities.

   > :warning: `TestModuleRules` overrides many default UBT flags from its base class `ModuleRules`. This reduces compilation bloat and minimizes compilation times for most tests out of the box. You can always override these defaults in your `TestModuleRules` derived class, but they should not be changed in `TestModuleRules` directly.

#### Test Target Rules Reference

The `TestTargetRules` class extends TargetRules with the following:
* `bUsePlatformFileStub`: This causes the platform-dependent `FPlatformFile` runtime instance to be replaced with a mock that disables IO operations. Use this to disable asset loading when testing against the engine module.\
  > **_NOTE_** Setting this property changes the value of `UE_LLT_USE_PLATFORM_FILE_STUB`, which tests can use to perform additional IO mocking. The `FPlatformFile` is saved using `SaveDefaultPlatformFile` and restored with `UseDefaultPlatformFile`, both of which require `#include "TestCommon/Initialization.h"`.
* `bMockEngineDefaults`: When testing with the engine module, certain resources are managed by default or loaded from asset files. These operations require cooking assets. Use this for tests that don't need to load assets; the effect is to mock engine default materials, world objects, and other resources.\
  > **_NOTE_** Setting this property changes the value of `UE_LLT_WITH_MOCK_ENGINE_DEFAULTS`.
* `bNeverCompileAgainstEngine`: The default behavior of the LLT Framework is to automatically set `bCompileAgainstEngine = true` whenever the `Engine` module is in the build dependency graph. This property can stop this behavior such that we're not compiling with the engine even if the engine module is in the graph.
* `bNeverCompileAgainstCoreUObject`: Same as *bNeverCompileAgainstEngine* but for `CoreUObject`.
* `bNeverCompileAgainstApplicationCore`: Same as *bNeverCompileAgainstEngine* but for `ApplicationCore`.
* `bNeverCompileAgainstEditor`: same as *bNeverCompileAgainstEngine* but for `UnrealEd`.
* `bWithLowLevelTestsOverride`: Set this to include implicit tests from depending modules. It forces the WITH_LOW_LEVEL_TESTS macro to be `true` which is normally reserved for implicit tests

> :warning: Just like `TestModuleRules`, `TestTargetRules` sets default UBT flags. Notably it disables UE features such UObjects, localizations, stats, and others.

### Engine tests

In this type of explicit test, the LLT framework compiles and runs explicit tests that include the engine module. Because loading assets requires cooking for most platforms, the engine module cannot be used, so engine tests only work with the following flags set in the `.Target.cs` file:
<font size="2">
```csharp
public UEModuleTestsTarget(TargetInfo Target) : base(Target)
{
  bUsePlatformFileStub = true;
  bMockEngineDefaults = true;
}
```
</font>

It is also necessary to initialize and cleanup Engine components.
To do this, provide a .cpp file containing initialization events:
<font size="2">
```cpp
#include "TestCommon/Initialization.h"

GROUP_BEFORE_ALL(Catch::DefaultGroup)
{
  InitAll(true, true);
}

GROUP_AFTER_ALL(Catch::DefaultGroup)
{
  CleanupAll();
}
```
</font>

## Implicit Tests

Implicit Tests reside in the same module that you want to test. All implicit tests are conditionally compiled through the `WITH_LOW_LEVEL_TESTS` definition. While they can be placed anywhere in the module, for example, in the **Private** folder, it can be hard to distinguish them from module source code. To ensure that your tests are discoverable, follow these conventions:
* Create a **Tests** folder in your module at the same level as the **Public** and **Private** folders.
* Mirror the module's folder structure inside **Tests** whenever there's a matching test for a source file.

Because there isn't an explicit target like explicit tests require, these tests need an existing target to be built with. This requires implicit tests to use Unreal Build Tool to produce test executables for implicit tests. For more information, see the [Implicit Tests](#implicit-tests) sub-section of [Build and Run Low-Level Tests](#build-and-run-low-level-tests).

### Directory Structure

Consider the following example with a module called **SampleModule** and the following directory structure:

```
.
└── SampleModule/
    ├── Public/
    │   └── SampleCode.h
    └── Private/
        ├── SampleCode.cpp
        └── SampleModuleSubDirectory/
            ├── MoreSampleCode.h
            └── MoreSampleCode.cpp
```
To add implicit tests to this module, create a directory titled **Tests** at the same level as the **Public** and **Private** directories and mirror the directory structure of the **Private** directory. For each C++ file, create a corresponding test file in the same relative location. The resulting structure looks like this:
```
.
└── SampleModule/
    ├── Public/
    │   └── SampleCode.h
    ├── Private/
    │   ├── SampleCode.cpp
    │   └── SampleModuleSubDirectory/
    │       ├── MoreSampleCode.h
    │       └── MoreSampleCode.cpp
    └── Tests/
        ├── SampleCodeTests.cpp
        └── SampleModuleSubDirectory/
            └── MoreSampleCodeTests.cpp
```
With this structure, the file `SampleCodeTests.cpp` contains tests for the `SampleCode.cpp` file, and `MoreSampleCodeTests.cpp` contains tests for the `MoreSampleCode.cpp` file.

Should your tests depend on Game-like configuration loading creating a SampleCodeTests.uproject in the root of the will allow the Test program to find the configuration folder with an additional `--projectdir=` paramter. 

```
.
└── SampleModule/
    ├── Public/
    ├── Private/
    └── Tests/
        ├── SampleCodeTests.cpp
        └── SampleModuleSubDirectory/
            └── MoreSampleCodeTests.cpp
        └── Config/
            └── DefaultEngine.ini
        └── SampleCodeTests.uproject
```

Note the `.Build.cs` should be updated to include `StagesWithProjectFile=True` to indicate to BuildGraph that we should use a Game-like folder structure when deploying these tests to run.


## Write Low-Level Tests

This section primarily discusses structure, guidelines, and best practices for writing **Low-Level Tests (LLTs)** with Catch2 in the context of **Unreal Engine (UE)**. See the [Catch2 GitHub Repository](https://github.com/catchorg/Catch2) for information specific to Catch2. For a complete guide on writing tests, see the [Catch2 Reference](https://github.com/catchorg/Catch2/tree/devel/docs).

>> **_TIP_**: Be sure to use Unreal C++ coding conventions for tests. See the [Unreal Coding Standard](https://docs.unrealengine.com/5.2/en-US/epic-cplusplus-coding-standard-for-unreal-engine/) for more information.

### Before You Begin

Review the [Types of Low-Level Tests](#types-of-low-level-tests) section for the following items:
* Determine whether an **Explicit** or **Implicit** test is right for your specific use-case.
* Ensure that your directories match the ones outlined in that document's steps.

Once you are ready to start writing test `.cpp` files, follow these steps:

1. Give your `.cpp` test file a descriptive name, such as `<NAME_OF_FILE>Test.cpp`
    * See the [Naming Conventions](#naming-conventions-for-files-and-folders) section of this document for more information.
2. If you are writing `Implicit Tests`, wrap your `.cpp` file in an `#if WITH_LOW_LEVEL_TESTS [...] #endif` directive.
    * See the [BDD Test Example](#behavior-driven-development-test-example) below.
3. Include all necessary header files.
    * At minimum, you need to have optional `#include "CoreMinimal.h"` and `#include "TestHarness.h"`.
	* After you include the minimum headers, include only the headers that are necessary to complete your tests.
4. Now you can write your test using either the **TDD (Test-Driven Development)** or **BDD (Behavior-Driven Development)** paradigms.
    * [BDD Test Example](#behavior-driven-development-test-example)
	* [TDD Test Example](#test-driven-development-test-example)

### Behavior Driven Development Test Example

BDD-style tests focus testing through a `SCENARIO`. A file can include multiple scenarios. The core structure of a scenario is:
* `GIVEN`: setup conditions
* `WHEN`: actions are performed
* `THEN`: expected result holds.

The `GIVEN` and `WHEN` sections can contain additional initialization and changes to internal state. The `THEN` section should perform checks to determine whether the desired result holds true. `CHECK` failures continue execution while `REQUIRE` stops execution of a single test.

The code example below provides a general outline of a BDD-style test in the Low-Level Tests framework in UE:

```cpp
#if WITH_LOW_LEVEL_TESTS // Required only for implicit tests!

#include "CoreMinimal.h"
#include "TestHarness.h"

// Other includes must be placed after CoreMinimal.h and TestHarness.h, grouped by scope (std libraries, UE modules, third party etc)

/* A BDD-style test */

SCENARIO("Summary of test scenario", "[functional][feature][slow]") // Tags are placed in brackets []
{
    GIVEN("Setup phase")
    {
        // Initialize variables, setup test preconditions etc

        [...]

        WHEN("I perform an action")
        {
            // Change internal state

            [...]

            THEN("Check for expectations")
            {
                REQUIRE(Condition_1);
                REQUIRE(Condition_2);
                // Not reached if previous require fails
                CHECK(Condition_3);
            }
        }
    }
}

#endif //WITH_LOW_LEVEL_TESTS
```

### Test Driven Development Test Example

TDD-style tests focus testing through a `TEST_CASE`. Each `TEST_CASE` can include code to set up the case being tested. The actual test case can then be broken down into multiple `SECTION` blocks. Each of the `SECTION` blocks performs checks to determine whether the desired result holds true. After all the checks are performed in `SECTION` blocks, the end of the `TEST_CASE` can include any necessary teardown code.

The code example below provides a general outline of a TDD-style test in the Low Level Tests framework in UE:

```cpp
#if WITH_LOW_LEVEL_TESTS // Required only for implicit tests!

#include "CoreMinimal.h"
#include "TestHarness.h"

// Other includes must be placed after CoreMinimal.h and TestHarness.h, grouped by scope (std libraries, UE modules, third party etc)

/* Classic TDD-style test */

TEST_CASE("Summary of test case", "[unit][feature][fast]")
{
    // Setup code for this test case

    [...]

    // Test can be divided into sections
    SECTION("Test #1")
    {
        REQUIRE(Condition_1);
    }

    ...

    SECTION("Test #n")
    {
        REQUIRE(Condition_n);
    }

    // Teardown code for this test case

    [...]

}

#endif //WITH_LOW_LEVEL_TESTS
```

> **_WARNING_**: The #if WITH_LOW_LEVEL_TESTS conditional check is reserved for compiling implicit tests. Explicit tests don't require this check but they can still include implicit tests from dependencies by setting bWithLowLevelTestsOverride = true in the target class: this flag forces WITH_LOW_LEVEL_TESTS to 1.

Test cases can also use double colon `::` notation to create a hierarchy in tests:

```
TEST_CASE("Organic::Tree::Apple::Throw an apple away from the tree") { ... }
```

The examples contained in this section are not exhaustive of all the features of Low-Level Tests or Catch2 in Unreal Engine. Generators, benchmarks, floating point approximation helpers, matchers, variable capturing, logging and more are all detailed in the external Catch2 Documentation.

### More Examples

There are several UE-specific Low-Level Test examples in the engine directory `Engine/Source/Runtime/Core/Tests`. To continue the example from the [Types of Low-Level Tests](#types-of-low-level-tests), you can see an example of TDD-style tests in the file `ArchiveReplaceObjectRefTests.cpp` located in `Engine/Source/Programs/LowLevelTests/FoundationTests/Tests`.

## Additional Low-Level Tests Features

### Test Groups and Lifecycle Events

Grouping tests is a feature of UE's extended Catch2 library. By default, all test cases are grouped under a group with an empty name. To add a test to a group, specify its name as the first parameter and use `GROUP_*` versions of test cases:

```cpp
GROUP_TEST_CASE("Apples", "Recipes::Baked::Pie::Cut slice", "[baking][recipe]") 
GROUP_TEST_CASE_METHOD("Oranges", OJFixture, "Recipes::Raw::Juice Oranges", "[raw][recipe]") 
GROUP_METHOD_AS_TEST_CASE("Pears", PoachInWine, "Recipes::Boiled::Poached Pears", "[desert][recipe]") 
GROUP_REGISTER_TEST_CASE("Runtime", UnregisteredStaticMethod, "Dynamic", "[dynamic]")
```

For each group there are six lifecycle events that are self descriptive. The following code section illustrates these events:

```cpp
GROUP_BEFORE_ALL("Apples") {
    std::cout << "Called once before all tests in group Apples, use for one-time setup.\n";
} 

GROUP_AFTER_ALL("Oranges") { 
    std::cout << "Called once after all tests in group Oranges, use for one-time cleanup.\n"; 
} 

GROUP_BEFORE_EACH("Apples") { 
    std::cout << "Called once before each test in group Apples, use for repeatable setup.\n"; 
} 

GROUP_AFTER_EACH("Oranges") { 
    std::cout << "Called once after each tests in group Oranges, use for repeatable cleanup.\n"; 
} 

GROUP_BEFORE_GLOBAL() { 
    std::cout << "Called once before all groups, use for global setup.\n"; 
} 

GROUP_AFTER_GLOBAL() { 
    std::cout << "Called once after all groups, use for global cleanup.\n"; 
} 

GROUP_TEST_CASE("Apples", "Test #1") { 
    std::cout << "Apple #1\n"; 
} 

GROUP_TEST_CASE("Apples", "Test #2") { 
    std::cout << "Apple #2\n"; 
} 

GROUP_TEST_CASE("Oranges", "Test #1") { 
    std::cout << "Orange #1\n"; 
} 

GROUP_TEST_CASE("Oranges", "Test #2") { 
    std::cout << "Orange #2\n"; 
}
```

This produces the output:

```
Called once before all groups, use for global setup. 
Called once before all tests in group Apples, use for one-time setup. 
Called once before each test in group Apples, use for repeatable setup. 
Apple #1. 
Called once before each test in group Apples, use for repeatable setup. 
Apple #2. 
Orange #1. 
Called once after each tests in group Oranges, use for repeatable cleanup. 
Orange #2. 
Called once after each tests in group Oranges, use for repeatable cleanup. 
Called once after all tests in group Oranges, use for one-time cleanup. 
Called once after all groups, use for global cleanup.
```

## Guidelines for Writing and Organizing Tests

### Naming Conventions for Files and Folders

* Give your test files descriptive names.
    * If `SourceFile.cpp` is the source file you want to test, name your test file `SourceFileTest.cpp` or `SourceFileTests.cpp`.
* Mirror the tested module's folder structure.
    * `Alpha/Omega/SourceFile.cpp` maps to `Tests/Alpha/Omega/SourceFileTests.cpp` for implicit tests or `Alpha/Omega/SourceFileTests.cpp` for explicit tests.

Avoid using terms derived from unit test in test file names if they aren't unit tests — this definition is restrictive and misnaming can cause confusion.

A unit test should target the smallest testable unit — a class or a method — and run in seconds or less. The same principle applies for any other type of specialized test — integration, functional, smoke, end to end, performance, stress or load test. You can also place all unit tests in a **Unit** subfolder.

#### Explicit Tests Resources Folder

Test files, such as arbitrary binary files, assets files, or any other file-system based resource, must be placed into a *resource folder* for explicit tests. Set this folder in the `.Build.cs` module:

```csharp
SetResourcesFolder("TestFilesResources");
```

When **Unreal Build Tool (UBT)** runs the platform deploy step, UBT copies this folder and its entire contents into the binary path so tests can relatively locate and load resources from it.

### Retrieve active test information

To access the name and tag list of current running test include `<catch2/catch_active_test.hpp>` in your test:
* `Catch::getActiveTestName()` will return the name of the current running test - also called the active test
* `Catch::getActiveTestTags()` will return the tags of the current running test in alphabetical order
```cpp
#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"

#include <catch2/catch_active_test.hpp>
#include <string>

TEST_CASE("Example::ActiveTest", "[GetTags][GetName][ActiveTest]")
{
    // Returns "Example::ActiveTest"
	const std::string CurrentTestName = Catch::getActiveTestName();

	// Returns "[ActiveTest][GetName][GetTags]"
	const std::string CurrentRunningTestTags = Catch::getActiveTestTags();
}

#endif
```

### Best Practices

* Provide tags to test cases and scenarios.
    * Use consistent names and keep them short.
	* Use tags to your advantage. For example, you can choose to parallelize the run of tests tagged `[unit]` or tag all slow running tests to be run on a nightly build.
* Ensure each `SECTION` or `THEN` block includes at least one `REQUIRE` or a `CHECK`.
    * Tests that don't have expectations are useless.
* Use `REQUIRE` when test preconditions must be satisfied.
    * `REQUIRE` immediately stops on failure but `CHECK` doesn't.
* Design tests that are deterministic and fit a certain type.
    * Create and group tests by type whether they are unit, integration, functional, stress test etc.
* Tag slow tests with `[slow]` or `[performance]` if they are intended as performance tests.
    * This can be used to filter them out into a nightly build in the Continuous Integration/Continuous Delivery (CI/CD) pipeline.
* Ensure test code supports all platforms that the tested module requires.
    * For example, when working with the platform file system, use the `FPlatformFileManager` class, don't assume the test will run exclusively on a desktop platform.
* Use test groups and lifecycle events to initialize certain tests independently from others.
    * Refer to the test groups and lifecycle events section.
* Follow best practices for each type of test.
    * For example, unit tests should use mocking and not rely on external dependencies (other modules, a local database etc) and should not have order dependencies.
    * See the Low-Level Tests Overview for more information on different types of tests and their characteristics.

## Build and Run Low-Level Tests

You can build and run low-level tests with:
* [Visual Studio](#visual-studio)
* [Unreal Build Tool (UBT)](#unreal-build-tool)
* [BuildGraph](#buildgraph)
* [BuildCookRun](#buildcookrun)

You can build and run explicit tests using any of these tools. We recommend you build and run explicit tests with BuildGraph whenever possible. Currently, you can only build and run implicit tests with Unreal Build Tool.

| Test Type Availability | Visual Studio | Unreal Build Tool | BuildGraph |
|------------------------|---------------|-------------------|------------|
| Implicit Tests         | No            | Yes               | No         |
| Explicit Tests         | Yes           | Yes               | Yes        |

At the end of this page, the [Example: Foundation Tests](#example-foundation-tests) section guides you through how to build and run a low-level tests project included with Unreal Engine.

### Visual Studio

You can build and run explicit tests directly from Visual Studio on desktop platforms:

1. Install UnrealVS.
    * This is optional, but strongly recommended, as it enhances test discoverability. For more information about UnrealVS, see the [UnrealVS Extension](https://docs.unrealengine.com/5.3/en-US/using-the-unrealvs-extension-for-unreal-engine-cplusplus-projects) documentation.
2. **Build** the test projects from Visual Studio to produce the executables.
    * The Visual Studio built-in test adapter discovers tests in the Catch2 executables. You can build with UnrealVS or directly through Visual Studio's interface.
3. The tests are displayed in the **Test Explorer**. Select **Test > Test Explorer** from the menu. From here, you can run tests and navigate to their source code.
4. If there are no tests in Test Explorer, it's likely that the build at step 2 didn't update the executable. Run **Rebuild** on the test project to remedy this problem.

### Unreal Build Tool

#### Build

All low level tests must be built using the UBT mode `-Mode=Test`. This mode performs a pre-analysis of the build target's dependency chain and detects UE core modules such as CoreUObject, Engine, ApplicationCore and UnrealEd and sets appropriate compilation flags. By default `-Mode=Test` is used to build explicit tests - tests that have their own explicitely defined target. To build an implicit test from an existing target use `-Mode=Test -Implicit`: note the additional -Implicit argument here.

##### Explicit Tests

You can use Unreal Build Tool to build explicit tests using `-Mode=Test`. Suppose we build explicit test cases with their target class `MyTestsTarget`:

```
.\RunUBT.bat MyTestsTarget Development Win64 -Mode=Test
```

The configuration used above is `Development` and the platform is `Win64` for example purposes. All configurations and platforms are supported.

##### Implicit Tests

To build implicit tests, use an existing target, for example `UnrealEditor`, and use `-Mode=Test -Implicit`, which builds a program target based on the given target:

```
.\RunUBT.bat UnrealEditor Development Win64 -Mode=Test -Implicit
```

Using test mode, the tests that are included in every module in the dependency graph are collected into one executable program. Unreal Build Tool is currently the only way to build implicit tests.

#### Run

The previous UBT commands build a test executable. The test executable is located in the same base folder that the target normally outputs, such as `Binaries/<PLATFORM>`, but under a folder with the same name as the target. Here is an example that runs a low-level tests executable from the command-line after building them in the manner above with Unreal Build Tool:

```
MyTests.exe --log --debug --sleep=5 --timeout=10 -r xml -# [#MyTestFile][Core] --extra-args -stdout
This command-line does the following:
```

LLT arguments:

```
--log --debug --sleep=5 --timeout=10
```
* Enable UE logging.
* Print low-level tests debug messages (test start, finish, completion time).
* Wait 5 seconds before running tests.
* Set a per-test timeout of 10 minutes.

Catch2 arguments:

```
-r xml -# [#MyTestFile][Core]
```
* Enable XML reporting.
* Use filenames as filter tags and select all tests from the file MyTestFile that are tagged [Core].

UE arguments:
```
--extra-args -stdout
```
* Set -stdout to the UE command-line.

#### Command-Line Reference

Once built, you can use a test executable for running pre-submit tests or as part of a Continuous Integration/Continuous Delivery (CI/CD) pipeline. The LLT executable supports a range of command line options that cover many use cases.

| Argument | Flag or Key-Value Pair | Description |
|----------|------------------------|-------------|
| --global-setup | Flag | Run global setup that initializes UE core components. |
| --no-global-setup | Flag | Use this to disable global setup. |
| --log | Flag | Enabled UE log output. |
| --no-log | Flag | Disabled UE log output. |
| --debug | Flag | Enable LowLevelTestsRunner logger debug messages for current test execution. |
| --mt | Flag | Set bMultiThreaded=true. Use this to configure a multithreaded environment. |
| --no-mt | Flag | Set bMultiThreaded=false. Use this to configure a single-threaded environment. |
| --wait | Flag | Wait for user input before exiting. |
| --no-wait | Flag | Do not wait for user input before exiting. This is the default behavior. |
| --attach-to-debugger, --waitfordebugger | Flag | Application waits for debugger to attach before the global setup phase. |
| --buildmachine | Flag | Set the UE global variable bIsBuildMachine=true. Used for development to control CI/CD behavior. |
| --sleep=<SECONDS> | Key-Value Pair | Set a sleep period in seconds before the global setup phase. Useful for cases where synchronization demands tests to wait before startup. |
| --timeout=<MINUTES> | Key-Value Pair | Set a per-test timeout in minutes. When the timeout is reached during a single test case, an error message is printed. |
| --reporter= etc. -r etc.| Both | Catch2 command-line options. Any command line option that is not one from the above, and it's not after --extra-args, is automatically sent to Catch2. For a full reference of the Catch2 command-line options, see the external [Catch2 Command-Line](https://github.com/catchorg/Catch2/blob/devel/docs/command-line.md#top) documentation in the Catch2 GitHub repository. |
| --extra-args | Flag | All arguments set after this option are set on UE's FCommandLine. Useful in cases where features are enabled from the command-line. |

> **_WARNING_** As described in the last two entries above, any argument that is not enumerated in the reference, up to `--extra-args` included, is sent directly to the Catch2 command line argument list.

### BuildGraph

The recommended way to build and run tests is through the BuildGraph script. A basic command looks like this:

```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Run Low Level Tests"
```

> **NOTE_** Windows, Mac and Linux can use similar commands.


The script located at `Engine/Build/LowLevelTests.xml` works with test metadata files. To generate metadata files, refer to the Generate BuildGraph Script Metadata files section of the Types of Low-Level Tests documentation. Correct execution of this script is conditional on successfully generated test metadata scripts, so be sure to confirm that files are produced in their expected locations. All the test metadata `.xml` files are included in `LowLevelTests.xml` and this metadata drives the execution of the nodes in the build graph.

Here are some common ways to use the BuildGraph script:
1. Run a test with name `MyTest` on Windows:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="MyTest Tests Win64"
```
2. Set a specific build configuration other than the default, which is Development. For example, you can set the configuration to **Debug** like this:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Foundation Tests Win64" -set:Configuration="Debug"
```
3. Build and launch a test in Debug configuration and wait for debugger to attach to it:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Foundation Tests Win64" -set:Configuration="Debug" -set:AttachToDebugger=true
```
4. If a platform's tooling supports deploying an application onto a device of a given name or IP, you can launch it onto that device. This command can also be used together with `AttachToDebugger` as well if the platform has remote debugging tools:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Foundation Tests Win64" -set:Device="<IP_OR_NAME_OF_DEVICE>"
```
5. Build Catch2 for a target platform:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Catch2 Build Library Win64"
```

### BuildCookRun

Engine provided script BuildCookRun provides a wrapping script to Unreal Build Tool allows you to build, cook, and run your application in one script. For workflows specific to only deploying low level tests and running them we recommend the following command. BuildCookRun can be used on Low Level Test programs that have `StagesWithProjectFile=true` specified in its metadata and .uproject files appropriately configured. 

It is important to pass `--projectprogramoverride` if your tests exist inside a Game's project folder for dependencies to correctly be passed to Unreal Build Tool.

```
.\RunUAT.bat BuildCookRun  -Build -NoBootstrapExe -NoCodeSign -SkipCook -Stage -NoSubmit -SkipBuildEditor=True -SkipClient -Archive -ArchiveDirectory=D:\build\SampleTestName -project=MyGame\Path\To\SampleTestName.uproject -configuration=Development -platform=Win64 --projectprogramoverride=MyGame\MyGame.uproject
```

If your Binaries folder within the Archive directory appears  subfolder inside your Game/Binaries/ folder, you will want to include a directory remapping so the correct Project folder will be found as well, otherwise the default UE5 system will not find content and configs.

```
[Staging]
;Remap platform folders for loading configs from these specific folders
+RemapDirectories=(From="SampleGame/Binaries/Win64/SampleTests",To="SampleTests/Binaries/Win64")
```

If we want to deploy to a device the addional `-Device=<Device.json>` paramter can be passed in for BuildCookRun to deploy your program to a device. For additional help on BuildCookRun see `-Help` or specific documenation pages on Automation Tool.


## Example: Foundation Tests

### Overview

The Foundation Tests project is located in the Visual Studio **Solution Explorer** under the `Programs/LowLevelTests` folder.

It's designed to reunite implicit tests from core UE modules, including but not limited to `Core`, `Cbor` and `CoreUObjects`. It's set up as an explicit test suite, but it allows implicit tests from dependent modules to be collected by setting the flag `bWithLowLevelTestsOverride = true` in the target file. For example, it collects the Core implicit tests from `Engine/Source/Runtime/Core/Tests`.

To add new tests to the Foundation Tests project, create implicit tests in any of its dependent modules. Tests are picked up from the whole dependency graph, not just the modules specified in the `.Build.cs` file. For example, the Core module depends on the DerivedDataCache module, which has tests defined in `Engine/Source/Developer/DerivedDataCache/Tests`. These are also collected into the final executable.

There are lifecycle events defined in `Tests/TestGroupEvents.cpp`. Be mindful of the order of execution of these events, as they impact correct execution of tests. Lack of setup, or incorrectly placed setup, can cause runtime errors, the same goes for teardown events.

### Build and Run

Build and run tests from Visual Studio or use BuildGraph.

#### Visual Studio

To build Foundation Tests from Visual Studio, follow these steps:

1. Ensure that you have installed **UnrealVS** as it makes building Tests easier.
    * See the [UnrealVS Extension](https://docs.unrealengine.com/5.3/en-US/using-the-unrealvs-extension-for-unreal-engine-cplusplus-projects) documentation for more information.
2. Navigate to the Visual Studio menu and find **Solution Configurations**.
3. From the **Solution Configurations** dropdown, select your desired configuration, for example **Development**.
4. In the UnrealVS toolbar, find the **Startup Project** dropdown, and select **FoundationTests**.
5. In the Visual Studio menu bar, select **Build > Build Solution**.

This builds Foundation Tests and its dependencies. To run Foundation Tests, navigate to the `Engine/Binaries/Win64/FoundationTests` directory from your terminal or command prompt and run the `FoundationTests.exe` executable with `./FoundationTests`.

If everything works correctly, you will see some Log text in your terminal window and, if all tests pass, a dialog at the end that reads "All tests passed…".

#### BuildGraph
To build and run Foundation Tests, navigate to your project directory and run the command:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Foundation Tests Win64"
```
You can specify different platforms, build configuration, device target to run tests on or make the tests wait for a debugger to attach.

## Build custom version of Catch2

The BuildGraph script LowLevelTests.xml offers additional support to build the third-party Catch2 library using cmake. The following steps describe how to build a custom version of Catch2 based on an official released version.

1. Download the desired version source code from https://github.com/catchorg/Catch2/releases and place it into  `Engine\Source\ThirdParty\Catch2` following the folder naming convention *vX.Y.Z* and apply source code changes.
2. For the next step make sure cmake is installed and added to PATH. The version should be the same as the UE supported version found in `Engine\Extras\ThirdPartyNotUE\CMake\bin`: run `cmake --version` in this folder.
3. Generate a VS2022 project using cmake command (example for version v3.4.0):
	```
	cmake.exe -B "<ROOT_DIR>\Engine\Source\ThirdParty\Catch2\v3.4.0\VSProject" -S "<ROOT_DIR>\Engine\Source\ThirdParty\Catch2\v3.4.0" -G "Visual Studio 17 2022" -DCMAKE_ARCHIVE_OUTPUT_DIRECTORY="<ROOT_DIR>\Engine\Source\ThirdParty\Catch2\v3.4.0\lib\Win64\x64" -DSET_CONFIG_NO_COLOUR_WIN32=OFF -A x64
	```
	The VS20022 project will be in the current Catch2 v3.4.0 folder under the `VSProject` directory.

4. Mirror the files from `VSProject\generated-includes` into `v3.4.0\src`, for example `VSProject\generated-includes\catch2\catch_user_config.hpp` will be copied to `v3.4.0\src\catch2\catch_user_config.hpp`
5. Generate amalgamated header and source files - the .hpp header file can be included directly as opposed to using a library. From the `extras` folder  run:
    ```
	python ..\tools\scripts\generateAmalgamatedFiles.py
	```
	Notice the updated *catch_amalgamated.cpp* and *catch_amalgamated.hpp* files, they should contain all source code changes applied on top of the downloaded source code.

To build for a specific platform run the command:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Catch2 Build <PLATFORM>"
```

The default cmake generator for all platforms is "Makefile" with the exception of Mac which uses "XCode" and Windows which uses "VS2019".
To specify a different generator, use the `Catch2LibVariation` option:
```
.\RunUAT.bat BuildGraph -Script="Engine/Build/LowLevelTests.xml" -Target="Catch2 Build Win64" -set:Catch2LibVariation=VS2022
```
*Example: build with VS2022 generator for the Win64 platform*
