// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"

#include <catch2/catch_active_test.hpp>

#include <string>

TEST_CASE("LowLevelTestsRunner::SelfTests::Skip", "[SelfTests]")
{
	SKIP("SKIP is expected to work all the time.");
	CHECK(false);// should never reach this
}

TEST_CASE("LowLevelTestsRunner::SelfTests::GetCurrentTestCaseInfo", "[SelfTests][GetCurrentTestCaseInfo]")
{
	const std::string CurrentRunningTestName = Catch::getActiveTestName();
	CHECK(CurrentRunningTestName.compare("LowLevelTestsRunner::SelfTests::GetCurrentTestCaseInfo") == 0);

	// Tags are returned in alphabetical order
	const std::string CurrentRunningTestTags = Catch::getActiveTestTags();
	CHECK(CurrentRunningTestTags.find("[GetCurrentTestCaseInfo]") != std::string::npos);
	CHECK(CurrentRunningTestTags.find("[SelfTests]") != std::string::npos);
}

#endif